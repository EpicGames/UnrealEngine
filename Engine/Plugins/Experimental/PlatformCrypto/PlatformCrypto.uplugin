{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Platform Cryptography Plugin",
	"Description": "Exposes a unified API for cryptography functionality provided by the platform, if available. Otherwise, interfaces with OpenSSL.",
	"Category": "Misc",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": true,
	"CanContainContent": false,
	"IsBetaVersion": false,
	"Installed": false,
	"SupportedPrograms": [ "UnrealPak", "UnrealPakEx", "IasTool" ],
	"Modules": [
		{
			"Name": "PlatformCrypto",
			"Type": "RuntimeAndProgram",
			"LoadingPhase": "EarliestPossible",
			"PlatformAllowList": [
				"Android",
				"IOS",
				"Mac",
				"Win64",
				"Linux",
				"LinuxArm64"
			],
			"PlatformDenyList": [
				"TVOS"
			],
			"ProgramAllowList": [ "UnrealPak", "UnrealPakEx", "IasTool" ]
		},
		{
			"Name": "PlatformCryptoTypes",
			"Type": "RuntimeAndProgram",
			"LoadingPhase": "EarliestPossible",
			"PlatformAllowList": [
				"Android",
				"IOS",
				"Mac",
				"Win64",
				"Linux",
				"LinuxArm64"
			],
			"PlatformDenyList": [
				"TVOS"
			]
		},
		{
			"Name": "PlatformCryptoContext",
			"Type": "RuntimeAndProgram",
			"LoadingPhase": "EarliestPossible",
			"PlatformAllowList": [
				"Android",
				"IOS",
				"Mac",
				"Win64",
				"Linux",
				"LinuxArm64"
			],
			"PlatformDenyList": [
				"TVOS"
			]
		}
	]
}