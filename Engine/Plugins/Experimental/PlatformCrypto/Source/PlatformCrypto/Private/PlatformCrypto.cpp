// Copyright Epic Games, Inc. All Rights Reserved.

#include "IPlatformCrypto.h"

class FPlatformCryptoModularFeature : public IEngineCrypto
{
public:

	FPlatformCryptoModularFeature()
	{
		IModularFeatures::Get().RegisterModularFeature(IEngineCrypto::GetFeatureName(), this);
	}

	virtual ~FPlatformCryptoModularFeature()
	{
		Cleanup();
	}

	virtual void Shutdown() override
	{
		Cleanup();
	}
	
	/** IEngineCrypto implementation */
	virtual FRSAKeyHandle CreateRSAKey(const TArrayView<const uint8> InPublicExponent, const TArrayView<const uint8> InPrivateExponent, const TArrayView<const uint8> InModulus) override
	{
		return GetContext()->CreateKey_RSA(InPublicExponent, InPrivateExponent, InModulus);
	}

	virtual void DestroyRSAKey(FRSAKeyHandle InKey) override
	{
		GetContext()->DestroyKey_RSA(InKey);
	}

	virtual int32 GetKeySize(FRSAKeyHandle InKey) override
	{
		return GetContext()->GetKeySize_RSA(InKey);
	}

	virtual int32 GetMaxDataSize(FRSAKeyHandle InKey) override
	{
		return GetContext()->GetMaxDataSize_RSA(InKey);
	}

	virtual int32 EncryptPublic(const TArrayView<const uint8> InSource, TArray<uint8>& OutDestination, FRSAKeyHandle InKey) override
	{
		return GetContext()->EncryptPublic_RSA(InSource, OutDestination, InKey);
	}

	virtual int32 EncryptPrivate(const TArrayView<const uint8> InSource, TArray<uint8>& OutDestination, FRSAKeyHandle InKey) override
	{
		return GetContext()->EncryptPrivate_RSA(InSource, OutDestination, InKey);
	}

	virtual int32 DecryptPublic(const TArrayView<const uint8> InSource, TArray<uint8>& OutDestination, FRSAKeyHandle InKey) override
	{
		return GetContext()->DecryptPublic_RSA(InSource, OutDestination, InKey);
	}

	virtual int32 DecryptPrivate(const TArrayView<const uint8> InSource, TArray<uint8>& OutDestination, FRSAKeyHandle InKey) override
	{
		return GetContext()->DecryptPrivate_RSA(InSource, OutDestination, InKey);
	}

private:
	void Cleanup()
	{
		Context.Reset();
	}

	TUniquePtr<FEncryptionContext>& GetContext()
	{
		if (!Context.IsValid())
		{
			Context = IPlatformCrypto::Get().CreateContext();
		}

		return Context;
	}

	/** Content used by IEngineCrypto implementation */
	TUniquePtr<FEncryptionContext> Context;
};

#if !defined(REQUIRE_EXPLICIT_GMALLOC_INIT) || !REQUIRE_EXPLICIT_GMALLOC_INIT
// This constructor during static initialization accesses FName before GMalloc is setup
FPlatformCryptoModularFeature GPlatformCryptoModularFeature;
#endif // !defined(REQUIRE_EXPLICIT_GMALLOC_INIT) || !REQUIRE_EXPLICIT_GMALLOC_INIT

IMPLEMENT_MODULE(IPlatformCrypto, PlatformCrypto)

void IPlatformCrypto::StartupModule()
{
	// The modular feature should really be registered here, but is up in the constructor and registered by global variable because of early needs for PAK decryption
}

void IPlatformCrypto::ShutdownModule()
{
#if !defined(REQUIRE_EXPLICIT_GMALLOC_INIT) || !REQUIRE_EXPLICIT_GMALLOC_INIT
	// Deallocate memory and unregister the feature during module shutdown to avoid static deinitialization issues with FName after memory has been shutdown
	GPlatformCryptoModularFeature.Shutdown();
	IModularFeatures::Get().UnregisterModularFeature(IEngineCrypto::GetFeatureName(), &GPlatformCryptoModularFeature);
#endif
}

TUniquePtr<FEncryptionContext> IPlatformCrypto::CreateContext()
{
	return MakeUnique<FEncryptionContext>();
}
