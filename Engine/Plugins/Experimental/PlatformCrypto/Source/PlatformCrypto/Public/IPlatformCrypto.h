// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"
#include "PlatformCryptoIncludes.h"

/**
 * Entry point for cryptographic functionality.
 */
class IPlatformCrypto : public IModuleInterface
{

public:

	/**
	 * Called when crytpo module is loaded
	 */
	virtual void StartupModule() override;

	/**
	 * Called when crytpo module is unloaded
	 */
	virtual void ShutdownModule() override;

	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline IPlatformCrypto& Get()
	{
		// Check whether the module is already loaded first. This allows Get() to be called on non-game threads.
		// LoadModuleChecked is not non-game-thread-safe.
		IPlatformCrypto* ModulePtr = FModuleManager::GetModulePtr<IPlatformCrypto>("PlatformCrypto");
		if (ModulePtr)
		{
			return *ModulePtr;
		}

		return FModuleManager::LoadModuleChecked<IPlatformCrypto>("PlatformCrypto");
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded( "PlatformCrypto" );
	}

	/**
	 * Creates and returns an encryption context object appropriate for the platform.
	 */
	PLATFORMCRYPTO_API TUniquePtr<FEncryptionContext> CreateContext();
};
