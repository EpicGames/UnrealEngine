// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class PlatformCryptoContext : ModuleRules
	{
		protected virtual bool DefaultToSSL { get { return true; } }
		protected virtual string PlatformEncryptionContextHeader { get { return "EncryptionContextOpenSSL.h"; } }

		public PlatformCryptoContext(ReadOnlyTargetRules Target) : base(Target)
		{
			// Platform specific implementations need to be compiled separately.
			bUseUnity = false;

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"PlatformCryptoTypes",
				}
			);

			if (DefaultToSSL)
			{
				PrivateDefinitions.Add($"PLATFORMCRYPTOCONTEXT_USE_OPENSSL=1");
				AddEngineThirdPartyPrivateStaticDependencies(Target, "OpenSSL");
			}
			else
			{
				PrivateDefinitions.Add($"PLATFORMCRYPTOCONTEXT_USE_OPENSSL=0");
			}
			PublicDefinitions.Add("PLATFORM_ENCRYPTION_CONTEXT_HEADER=\"" + PlatformEncryptionContextHeader + "\"");		
		}
	}
}


