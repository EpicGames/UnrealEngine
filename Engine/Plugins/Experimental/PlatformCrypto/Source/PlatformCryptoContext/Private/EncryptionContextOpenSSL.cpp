// Copyright Epic Games, Inc. All Rights Reserved.

#if PLATFORMCRYPTOCONTEXT_USE_OPENSSL

#include "EncryptionContextOpenSSL.h"
#include "PlatformCryptoAesEncryptorsOpenSSL.h"
#include "PlatformCryptoAesDecryptorsOpenSSL.h"
#include "PlatformCryptoUtilsOpenSSL.h"

#include "Containers/StringConv.h"
#include "Misc/ScopeExit.h"

THIRD_PARTY_INCLUDES_START
#include <openssl/bn.h>
#include <openssl/obj_mac.h>
#include <openssl/opensslv.h>
#include <openssl/rand.h>
#include <openssl/rsa.h>
#include <openssl/sha.h>
#include <openssl/pem.h>
#include <openssl/err.h>
THIRD_PARTY_INCLUDES_END

DEFINE_LOG_CATEGORY(LogPlatformCryptoOpenSSL);

struct FSHA256HasherOpenSSL::FImplDetails
{
	SHA256_CTX Ctx;
	enum class EState { NeedsInit, Ready, ReadyNeedsFinalize } State = EState::NeedsInit;
};

FSHA256HasherOpenSSL::FSHA256HasherOpenSSL() : Inner(MakePimpl<FSHA256HasherOpenSSL::FImplDetails>())
{
	check(Inner);
	Init();
}

EPlatformCryptoResult FSHA256HasherOpenSSL::Init()
{
	if (Inner->State == FImplDetails::EState::ReadyNeedsFinalize)
	{
		UE_LOGF(LogPlatformCryptoOpenSSL, Warning, "FSHA256Hasher::Init was called while there was still a digest being computed.");
		return EPlatformCryptoResult::Failure;
	}
	if (Inner->State == FImplDetails::EState::Ready)
	{
		return EPlatformCryptoResult::Success;
	}

	SHA256_Init(&Inner->Ctx);
	Inner->State = FImplDetails::EState::Ready;
	return EPlatformCryptoResult::Success;
}

EPlatformCryptoResult FSHA256HasherOpenSSL::Update(const TArrayView<const uint8> InDataBuffer)
{
	if (Inner->State == FImplDetails::EState::NeedsInit)
	{
		UE_LOGF(LogPlatformCryptoOpenSSL, Warning, "FSHA256Hasher::Update was called after the hasher was finalized without first calling Init");
		return EPlatformCryptoResult::Failure;
	}

	SHA256_Update(&Inner->Ctx, InDataBuffer.GetData(), static_cast<size_t>(InDataBuffer.Num()));
	Inner->State = FImplDetails::EState::ReadyNeedsFinalize;
	return EPlatformCryptoResult::Success;
}

EPlatformCryptoResult FSHA256HasherOpenSSL::Finalize(const TArrayView<uint8> OutDataBuffer)
{
	if (Inner->State == FImplDetails::EState::NeedsInit)
	{
		UE_LOGF(LogPlatformCryptoOpenSSL, Warning, "FSHA256Hasher::Finalize was called after the hasher was finalized without first calling Init");
		return EPlatformCryptoResult::Failure;
	}
	if (OutDataBuffer.Num() < (int32)OutputByteLength)
	{
		UE_LOGF(LogPlatformCryptoOpenSSL, Warning, "FSHA256Hasher::Finalize called with a buffer that is too small");
		return EPlatformCryptoResult::Failure;
	}

	SHA256_Final(reinterpret_cast<unsigned char*>(OutDataBuffer.GetData()), &Inner->Ctx);
	Inner->State = FImplDetails::EState::NeedsInit;
	return EPlatformCryptoResult::Success;
}

struct FEd25519CheckContextOpenSSL
{
	void Shutdown()
	{
		if (EvpPkey != nullptr)
		{
			EVP_PKEY_free(EvpPkey);
			EvpPkey = nullptr;
		}
		if (EvpMdCtx != nullptr)
		{
			EVP_MD_CTX_free(EvpMdCtx);
			EvpMdCtx = nullptr;
		}

		State = EState::NeedsInit;
	}

	bool IsReady() const
	{
		return State == EState::Ready;
	}

	EVP_MD_CTX* EvpMdCtx = nullptr;
	EVP_PKEY* EvpPkey = nullptr;

	enum class EState { NeedsInit, Ready, InitFailed } State = EState::NeedsInit;

protected:
	~FEd25519CheckContextOpenSSL()
	{
		Shutdown();
	}
};

struct FSignerEd25519OpenSSL::FImplDetails : public FEd25519CheckContextOpenSSL
{
};

FSignerEd25519OpenSSL::FSignerEd25519OpenSSL() : Inner(MakePimpl<FSignerEd25519OpenSSL::FImplDetails>())
{
	check(Inner);
}

FSignerEd25519OpenSSL::~FSignerEd25519OpenSSL()
{
}

bool FSignerEd25519OpenSSL::IsReady() const
{
	return Inner->IsReady();
}

bool FSignerEd25519OpenSSL::SetPrivateKey(const TArrayView<const uint8> Key)
{
	Inner->Shutdown();

	Inner->State = FImplDetails::EState::InitFailed;

	Inner->EvpMdCtx = EVP_MD_CTX_new();
	if (Inner->EvpMdCtx)
	{
		Inner->EvpPkey = EVP_PKEY_new_raw_private_key(EVP_PKEY_ED25519, NULL, Key.GetData(), Key.Num());
		if (Inner->EvpPkey)
		{
			if (EVP_DigestSignInit(Inner->EvpMdCtx, NULL, NULL, NULL, Inner->EvpPkey) == 1)
			{
				Inner->State = FImplDetails::EState::Ready;
			}
		}
	}

	return Inner->IsReady();
}

bool FSignerEd25519OpenSSL::Sign(const TArrayView<const uint8> Message, const TArrayView<uint8> Signature)
{
	if (!Inner->IsReady() || Signature.Num() != OutputSignatureLength)
	{
		return false;
	}

	size_t SignatureLength = OutputSignatureLength;

	bool Succeeded = EVP_DigestSign(Inner->EvpMdCtx, Signature.GetData(), &SignatureLength, Message.GetData(), Message.Num()) == 1;
	if (!Succeeded)
	{
		Inner->State = FImplDetails::EState::InitFailed;
	}

	return Succeeded;
}

struct FVerifierEd25519OpenSSL::FImplDetails : public FEd25519CheckContextOpenSSL
{
};

FVerifierEd25519OpenSSL::FVerifierEd25519OpenSSL() : Inner(MakePimpl<FVerifierEd25519OpenSSL::FImplDetails>())
{
	check(Inner);
}

FVerifierEd25519OpenSSL::~FVerifierEd25519OpenSSL()
{
}

bool FVerifierEd25519OpenSSL::IsReady() const
{
	return Inner->IsReady();
}

bool FVerifierEd25519OpenSSL::SetPublicKey(const TArrayView<const uint8> Key)
{
	Inner->Shutdown();

	Inner->State = FImplDetails::EState::InitFailed;

	Inner->EvpMdCtx = EVP_MD_CTX_new();
	if (Inner->EvpMdCtx)
	{
		Inner->EvpPkey = EVP_PKEY_new_raw_public_key(EVP_PKEY_ED25519, NULL, Key.GetData(), Key.Num());
		if (Inner->EvpPkey)
		{
			if (EVP_DigestVerifyInit(Inner->EvpMdCtx, NULL, NULL, NULL, Inner->EvpPkey) == 1)
			{
				Inner->State = FImplDetails::EState::Ready;
			}
		}
	}

	return Inner->IsReady();
}

bool FVerifierEd25519OpenSSL::Verify(const TArrayView<const uint8> Message, const TArrayView<const uint8> Signature)
{
	if (!Inner->IsReady())
	{
		return false;
	}

	return EVP_DigestVerify(Inner->EvpMdCtx, Signature.GetData(), Signature.Num(), Message.GetData(), Message.Num()) == 1;
}

TOptional<FCryptoKeyPair> FCryptoKeyGeneratorOpenSSL::GenerateEd25519() const
{
	TOptional<FCryptoKeyPair> Result;

	const FScopedEvpPKeyContext Context(EVP_PKEY_ED25519);
	if (!Context.Get())
	{
		UE_LOGF(LogPlatformCryptoOpenSSL, Error, "CryptoKeyGeneratorOpenSSL::GenerateEd25519. Unable to allocates public key algorithm context");
		return Result;
	}

	int ErrCode = EVP_PKEY_keygen_init(Context.Get());
	if (ErrCode != 1)
	{
		UE_LOGF(LogPlatformCryptoOpenSSL, Error, "CryptoKeyGeneratorOpenSSL::GenerateEd25519. Unable to initialize a public key algorithm context. ErrCode=[%d]", ErrCode);
		return Result;
	}

	EVP_PKEY* PvtKey = nullptr;
	ON_SCOPE_EXIT
	{
		if (PvtKey != nullptr)
		{
			EVP_PKEY_free(PvtKey);
		}
	};
	ErrCode = EVP_PKEY_keygen(Context.Get(), &PvtKey);
	if (ErrCode != 1)
	{
		UE_LOGF(LogPlatformCryptoOpenSSL, Error, "CryptoKeyGeneratorOpenSSL::GenerateEd25519. Unable to generate keys. ErrCode=[%d]", ErrCode);
		return Result;
	}

	FCryptoKeyPair KeyPair;
	KeyPair.PrivateKey.SetNumZeroed(32);
	size_t PrivateKeyLen = KeyPair.PrivateKey.Num();
	ErrCode = EVP_PKEY_get_raw_private_key(PvtKey, KeyPair.PrivateKey.GetData(), &PrivateKeyLen);
	if (ErrCode != 1)
	{
		UE_LOGF(LogPlatformCryptoOpenSSL, Error, "CryptoKeyGeneratorOpenSSL::GenerateEd25519. Unable to get private key. ErrCode=[%d] PrivateKeyLen=[%llu]", ErrCode, static_cast<uint64_t>(PrivateKeyLen));
		return Result;
	}

	KeyPair.PublicKey.SetNumZeroed(32);
	size_t PublicKeyLen = KeyPair.PublicKey.Num();
	ErrCode = EVP_PKEY_get_raw_public_key(PvtKey, KeyPair.PublicKey.GetData(), &PublicKeyLen);
	if (ErrCode != 1)
	{
		UE_LOGF(LogPlatformCryptoOpenSSL, Error, "CryptoKeyGeneratorOpenSSL::GenerateEd25519. Unable to get public key. ErrCode=[%d] PublicKeyLen=[%llu]", ErrCode, static_cast<uint64_t>(PublicKeyLen));
		return Result;
	}

	return Result.Emplace(MoveTemp(KeyPair));
}

TArray<uint8> FEncryptionContextOpenSSL::Encrypt_AES_256_ECB(const TArrayView<const uint8> Plaintext, const TArrayView<const uint8> Key, EPlatformCryptoResult& OutResult)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("OpenSSL AES256 ECB Encrypt"), STAT_OpenSSL_AES256_ECB_Encrypt, STATGROUP_PlatformCrypto);

	OutResult = EPlatformCryptoResult::Failure;

	TUniquePtr<IPlatformCryptoEncryptor> Encryptor = CreateEncryptor_AES_256_ECB(Key);
	if (!Encryptor.IsValid())
	{
		return TArray<uint8>();
	}

	TArray<uint8> Ciphertext;
	Ciphertext.AddUninitialized(Encryptor->GetUpdateBufferSizeBytes(Plaintext) + Encryptor->GetFinalizeBufferSizeBytes());

	int32 UpdateBytesWritten = 0;
	if (Encryptor->Update(Plaintext, Ciphertext, UpdateBytesWritten) != EPlatformCryptoResult::Success)
	{
		return TArray<uint8>();
	}

	int32 FinalizeBytesWritten = 0;
	if (Encryptor->Finalize(TArrayView<uint8>(Ciphertext.GetData() + UpdateBytesWritten, Ciphertext.Num() - UpdateBytesWritten), FinalizeBytesWritten) != EPlatformCryptoResult::Success)
	{
		return TArray<uint8>();
	}

	// Truncate message to final length
	Ciphertext.SetNum(UpdateBytesWritten + FinalizeBytesWritten);

	OutResult = EPlatformCryptoResult::Success;
	return Ciphertext;
}

TArray<uint8> FEncryptionContextOpenSSL::Encrypt_AES_256_CBC(const TArrayView<const uint8> Plaintext, const TArrayView<const uint8> Key, const TArrayView<const uint8> InitializationVector, EPlatformCryptoResult& OutResult)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("OpenSSL AES256 CBC Encrypt"), STAT_OpenSSL_AES256_CBC_Encrypt, STATGROUP_PlatformCrypto);

	OutResult = EPlatformCryptoResult::Failure;

	TUniquePtr<IPlatformCryptoEncryptor> Encryptor = CreateEncryptor_AES_256_CBC(Key, InitializationVector);
	if (!Encryptor.IsValid())
	{
		return TArray<uint8>();
	}

	TArray<uint8> Ciphertext;
	Ciphertext.AddUninitialized(Encryptor->GetUpdateBufferSizeBytes(Plaintext) + Encryptor->GetFinalizeBufferSizeBytes());

	int32 UpdateBytesWritten = 0;
	if (Encryptor->Update(Plaintext, Ciphertext, UpdateBytesWritten) != EPlatformCryptoResult::Success)
	{
		return TArray<uint8>();
	}

	int32 FinalizeBytesWritten = 0;
	if (Encryptor->Finalize(TArrayView<uint8>(Ciphertext.GetData() + UpdateBytesWritten, Ciphertext.Num() - UpdateBytesWritten), FinalizeBytesWritten) != EPlatformCryptoResult::Success)
	{
		return TArray<uint8>();
	}

	// Truncate message to final length
	Ciphertext.SetNum(UpdateBytesWritten + FinalizeBytesWritten);

	OutResult = EPlatformCryptoResult::Success;
	return Ciphertext;
}

TArray<uint8> FEncryptionContextOpenSSL::Encrypt_AES_256_GCM(const TArrayView<const uint8> Plaintext, const TArrayView<const uint8> Key, const TArrayView<const uint8> Nonce, TArray<uint8>& OutAuthTag, EPlatformCryptoResult& OutResult)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("OpenSSL AES256 GCM Encrypt"), STAT_OpenSSL_AES256_GCM_Encrypt, STATGROUP_PlatformCrypto);

	OutAuthTag.Reset();
	OutResult = EPlatformCryptoResult::Failure;

	TUniquePtr<IPlatformCryptoEncryptor> Encryptor = CreateEncryptor_AES_256_GCM(Key, Nonce);
	if (!Encryptor.IsValid())
	{
		return TArray<uint8>();
	}

	TArray<uint8> Ciphertext;
	Ciphertext.AddUninitialized(Encryptor->GetUpdateBufferSizeBytes(Plaintext) + Encryptor->GetFinalizeBufferSizeBytes());

	int32 UpdateBytesWritten = 0;
	if (Encryptor->Update(Plaintext, Ciphertext, UpdateBytesWritten) != EPlatformCryptoResult::Success)
	{
		return TArray<uint8>();
	}

	int32 FinalizeBytesWritten = 0;
	if (Encryptor->Finalize(TArrayView<uint8>(Ciphertext.GetData() + UpdateBytesWritten, Ciphertext.Num() - UpdateBytesWritten), FinalizeBytesWritten) != EPlatformCryptoResult::Success)
	{
		return TArray<uint8>();
	}

	// Truncate message to final length
	Ciphertext.SetNum(UpdateBytesWritten + FinalizeBytesWritten);

	OutAuthTag.SetNumUninitialized(Encryptor->GetCipherAuthTagSizeBytes());
	int32 AuthTagBytesWritten = 0;
	if (Encryptor->GenerateAuthTag(OutAuthTag, AuthTagBytesWritten) != EPlatformCryptoResult::Success)
	{
		OutAuthTag.Reset();
		return TArray<uint8>();
	}

	// Truncate auth tag to final length
	OutAuthTag.SetNum(AuthTagBytesWritten);

	OutResult = EPlatformCryptoResult::Success;
	return Ciphertext;
}

TArray<uint8> FEncryptionContextOpenSSL::Decrypt_AES_256_ECB(const TArrayView<const uint8> Ciphertext, const TArrayView<const uint8> Key, EPlatformCryptoResult& OutResult)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("OpenSSL AES256 ECB Decrypt"), STAT_OpenSSL_AES256_ECB_Decrypt, STATGROUP_PlatformCrypto);

	OutResult = EPlatformCryptoResult::Failure;

	TUniquePtr<IPlatformCryptoDecryptor> Decryptor = CreateDecryptor_AES_256_ECB(Key);
	if (!Decryptor.IsValid())
	{
		return TArray<uint8>();
	}

	TArray<uint8> Plaintext;
	Plaintext.AddUninitialized(Decryptor->GetUpdateBufferSizeBytes(Ciphertext) + Decryptor->GetFinalizeBufferSizeBytes());

	int32 UpdateBytesWritten = 0;
	if (Decryptor->Update(Ciphertext, Plaintext, UpdateBytesWritten) != EPlatformCryptoResult::Success)
	{
		return TArray<uint8>();
	}

	int32 FinalizeBytesWritten = 0;
	if (Decryptor->Finalize(TArrayView<uint8>(Plaintext.GetData() + UpdateBytesWritten, Plaintext.Num() - UpdateBytesWritten), FinalizeBytesWritten) != EPlatformCryptoResult::Success)
	{
		return TArray<uint8>();
	}

	// Truncate message to final length
	Plaintext.SetNum(UpdateBytesWritten + FinalizeBytesWritten);

	OutResult = EPlatformCryptoResult::Success;
	return Plaintext;
}

TArray<uint8> FEncryptionContextOpenSSL::Decrypt_AES_256_CBC(const TArrayView<const uint8> Ciphertext, const TArrayView<const uint8> Key, const TArrayView<const uint8> InitializationVector, EPlatformCryptoResult& OutResult)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("OpenSSL AES256 CBC Decrypt"), STAT_OpenSSL_AES256_CBC_Decrypt, STATGROUP_PlatformCrypto);

	OutResult = EPlatformCryptoResult::Failure;

	TUniquePtr<IPlatformCryptoDecryptor> Decryptor = CreateDecryptor_AES_256_CBC(Key, InitializationVector);
	if (!Decryptor.IsValid())
	{
		return TArray<uint8>();
	}

	TArray<uint8> Plaintext;
	Plaintext.AddUninitialized(Decryptor->GetUpdateBufferSizeBytes(Ciphertext) + Decryptor->GetFinalizeBufferSizeBytes());

	int32 UpdateBytesWritten = 0;
	if (Decryptor->Update(Ciphertext, Plaintext, UpdateBytesWritten) != EPlatformCryptoResult::Success)
	{
		return TArray<uint8>();
	}

	int32 FinalizeBytesWritten = 0;
	if (Decryptor->Finalize(TArrayView<uint8>(Plaintext.GetData() + UpdateBytesWritten, Plaintext.Num() - UpdateBytesWritten), FinalizeBytesWritten) != EPlatformCryptoResult::Success)
	{
		return TArray<uint8>();
	}

	// Truncate message to final length
	Plaintext.SetNum(UpdateBytesWritten + FinalizeBytesWritten);

	OutResult = EPlatformCryptoResult::Success;
	return Plaintext;
}

TArray<uint8> FEncryptionContextOpenSSL::Decrypt_AES_256_GCM(const TArrayView<const uint8> Ciphertext, const TArrayView<const uint8> Key, const TArrayView<const uint8> Nonce, const TArrayView<const uint8> AuthTag, EPlatformCryptoResult& OutResult)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("OpenSSL AES256 GCM Decrypt"), STAT_OpenSSL_AES256_GCM_Decrypt, STATGROUP_PlatformCrypto);

	OutResult = EPlatformCryptoResult::Failure;

	TUniquePtr<IPlatformCryptoDecryptor> Decryptor = CreateDecryptor_AES_256_GCM(Key, Nonce, AuthTag);
	if (!Decryptor.IsValid())
	{
		return TArray<uint8>();
	}

	TArray<uint8> Plaintext;
	Plaintext.AddUninitialized(Decryptor->GetUpdateBufferSizeBytes(Ciphertext) + Decryptor->GetFinalizeBufferSizeBytes());

	int32 UpdateBytesWritten = 0;
	if (Decryptor->Update(Ciphertext, Plaintext, UpdateBytesWritten) != EPlatformCryptoResult::Success)
	{
		return TArray<uint8>();
	}

	int32 FinalizeBytesWritten = 0;
	if (Decryptor->Finalize(TArrayView<uint8>(Plaintext.GetData() + UpdateBytesWritten, Plaintext.Num() - UpdateBytesWritten), FinalizeBytesWritten) != EPlatformCryptoResult::Success)
	{
		return TArray<uint8>();
	}

	// Truncate message to final length
	Plaintext.SetNum(UpdateBytesWritten + FinalizeBytesWritten);

	OutResult = EPlatformCryptoResult::Success;
	return Plaintext;
}

TUniquePtr<IPlatformCryptoEncryptor> FEncryptionContextOpenSSL::CreateEncryptor_AES_256_ECB(const TArrayView<const uint8> Key)
{
	return FPlatformCryptoEncryptor_AES_256_ECB_OpenSSL::Create(Key);
}

TUniquePtr<IPlatformCryptoEncryptor> FEncryptionContextOpenSSL::CreateEncryptor_AES_256_CBC(const TArrayView<const uint8> Key, const TArrayView<const uint8> InitializationVector)
{
	return FPlatformCryptoEncryptor_AES_256_CBC_OpenSSL::Create(Key, InitializationVector);
}

TUniquePtr<IPlatformCryptoEncryptor> FEncryptionContextOpenSSL::CreateEncryptor_AES_256_GCM(const TArrayView<const uint8> Key, const TArrayView<const uint8> Nonce)
{
	return FPlatformCryptoEncryptor_AES_256_GCM_OpenSSL::Create(Key, Nonce);
}

TUniquePtr<IPlatformCryptoDecryptor> FEncryptionContextOpenSSL::CreateDecryptor_AES_256_ECB(const TArrayView<const uint8> Key)
{
	return FPlatformCryptoDecryptor_AES_256_ECB_OpenSSL::Create(Key);
}

TUniquePtr<IPlatformCryptoDecryptor> FEncryptionContextOpenSSL::CreateDecryptor_AES_256_CBC(const TArrayView<const uint8> Key, const TArrayView<const uint8> InitializationVector)
{
	return FPlatformCryptoDecryptor_AES_256_CBC_OpenSSL::Create(Key, InitializationVector);
}

TUniquePtr<IPlatformCryptoDecryptor> FEncryptionContextOpenSSL::CreateDecryptor_AES_256_GCM(const TArrayView<const uint8> Key, const TArrayView<const uint8> Nonce, const TArrayView<const uint8> AuthTag)
{
	return FPlatformCryptoDecryptor_AES_256_GCM_OpenSSL::Create(Key, Nonce, AuthTag);
}

bool FEncryptionContextOpenSSL::DigestSign_RS256(const TArrayView<const uint8> Message, TArray<uint8>& Signature, FRSAKeyHandle Key)
{
	Signature.SetNum(GetKeySize_RSA(Key));

	unsigned int BytesWritten = 0;
	return RSA_sign(NID_sha256, Message.GetData(), Message.Num(), Signature.GetData(), &BytesWritten, (RSA*)Key) == 1;
}

bool FEncryptionContextOpenSSL::DigestVerify_PS256(const TArrayView<const char> Message, const TArrayView<const uint8> Signature, const TArrayView<const uint8> PKCS1Key)
{
	FScopedEVPMDContext Context;

	if (Context.Get() == nullptr)
	{
		return false;
	}

	const unsigned char* PKCS1KeyData = PKCS1Key.GetData();
	RSA* RsaKey = d2i_RSAPublicKey(nullptr, &PKCS1KeyData, PKCS1Key.Num());
	EVP_PKEY* PKey = EVP_PKEY_new();
	EVP_PKEY_assign_RSA(PKey, RsaKey);

	EVP_PKEY_CTX* KeyContext = nullptr;
	EVP_DigestVerifyInit(Context.Get(), &KeyContext, EVP_sha256(), nullptr, PKey);
	EVP_PKEY_CTX_set_rsa_padding(KeyContext, RSA_PKCS1_PSS_PADDING);
	EVP_DigestVerifyUpdate(Context.Get(), Message.GetData(), Message.Num());
	return EVP_DigestVerifyFinal(Context.Get(), const_cast<uint8*>(Signature.GetData()), Signature.Num()) == 1;
}

bool FEncryptionContextOpenSSL::DigestVerify_RS256(const TArrayView<const uint8> Message, const TArrayView<const uint8> Signature, FRSAKeyHandle Key)
{
	const int VerifyResult = RSA_verify(NID_sha256, Message.GetData(), Message.Num(), Signature.GetData(), Signature.Num(), (RSA*)Key);

	if (VerifyResult != 1)
	{
		UE_LOGF(LogPlatformCryptoOpenSSL, Error,
			"[FEncryptionContextOpenSSL::DigestVerify_RS256] Signature is invalid: (%lu): %ls.",
				ERR_get_error(), UTF8_TO_TCHAR(ERR_error_string(ERR_get_error(), nullptr)));
	}

	return VerifyResult == 1;
}


bool FEncryptionContextOpenSSL::GenerateKey_RSA(const int32 InNumKeyBits, TArray<uint8>& OutPublicExponent, TArray<uint8>& OutPrivateExponent, TArray<uint8>& OutModulus)
{
	if (InNumKeyBits % 8 != 0)
	{
		UE_LOGF(LogPlatformCryptoOpenSSL, Warning, "FEncryptionContextOpenSSL::GenerateKey_RSA failed because InNumKeyBits is not divisible by 8.");
		return false;
	}

	int32 KeySize = InNumKeyBits;

	RSA* RSAKey = RSA_new();
	BIGNUM* E = BN_new();
	BN_set_word(E, RSA_F4);
	RSA_generate_key_ex(RSAKey, KeySize, E, nullptr);

#if USE_LEGACY_OPENSSL
	const BIGNUM* PublicModulus = RSAKey->n;
	const BIGNUM* PublicExponent = RSAKey->e;
	const BIGNUM* PrivateExponent = RSAKey->d;
#else
	const BIGNUM* PublicModulus = RSA_get0_n(RSAKey);
	const BIGNUM* PublicExponent = RSA_get0_e(RSAKey);
	const BIGNUM* PrivateExponent = RSA_get0_d(RSAKey);
#endif

	FPlatformCryptoUtilsOpenSSL::BigNumToArray(PublicModulus, OutModulus);
	FPlatformCryptoUtilsOpenSSL::BigNumToArray(PublicExponent, OutPublicExponent);
	FPlatformCryptoUtilsOpenSSL::BigNumToArray(PrivateExponent, OutPrivateExponent);

	RSA_free(RSAKey);

	return true;
}

FRSAKeyHandle FEncryptionContextOpenSSL::CreateKey_RSA(const TArrayView<const uint8> PublicExponent, const TArrayView<const uint8> PrivateExponent, const TArrayView<const uint8> Modulus)
{
	RSA* NewKey = RSA_new();

	BIGNUM* BN_PublicExponent = PublicExponent.Num() > 0 ? BN_new() : nullptr;
	BIGNUM* BN_PrivateExponent = PrivateExponent.Num() > 0 ? BN_new() : nullptr;
	BIGNUM* BN_Modulus = BN_new();

	if (PublicExponent.Num())
	{
		FPlatformCryptoUtilsOpenSSL::LoadBinaryIntoBigNum(
			PublicExponent.GetData(), PublicExponent.Num(), BN_PublicExponent);
	}

	if (PrivateExponent.Num())
	{
		FPlatformCryptoUtilsOpenSSL::LoadBinaryIntoBigNum(
			PrivateExponent.GetData(), PrivateExponent.Num(), BN_PrivateExponent);
	}

	FPlatformCryptoUtilsOpenSSL::LoadBinaryIntoBigNum(
		Modulus.GetData(), Modulus.Num(), BN_Modulus);

#if USE_LEGACY_OPENSSL
	NewKey->n = BN_Modulus;
	NewKey->e = BN_PublicExponent;
	NewKey->d = BN_PrivateExponent;
#else
	RSA_set0_key(NewKey, BN_Modulus, BN_PublicExponent, BN_PrivateExponent);
#endif

	return NewKey;
}

FRSAKeyHandle FEncryptionContextOpenSSL::GetPublicKey_RSA(const FStringView PemSource)
{
	auto ConvertedSource = StringCast<UTF8CHAR>(PemSource.GetData());

	BIO* KeyBuf = BIO_new_mem_buf(ConvertedSource.Get(), ConvertedSource.Length());
	RSA* Key = PEM_read_bio_RSA_PUBKEY(KeyBuf, 0, 0, 0);

	BIO_free(KeyBuf);

	return Key;
}

void FEncryptionContextOpenSSL::DestroyKey_RSA(FRSAKeyHandle Key)
{
	RSA_free((RSA*)Key);
}

int32 FEncryptionContextOpenSSL::GetKeySize_RSA(FRSAKeyHandle Key)
{
	return RSA_size((RSA*)Key);
}

int32 FEncryptionContextOpenSSL::GetMaxDataSize_RSA(FRSAKeyHandle Key)
{
	return (GetKeySize_RSA(Key)) - RSA_PKCS1_PADDING_SIZE;
}

int32 FEncryptionContextOpenSSL::EncryptPublic_RSA(TArrayView<const uint8> Source, TArray<uint8>& Dest, FRSAKeyHandle Key)
{
	Dest.SetNum(GetKeySize_RSA(Key));
	int NumEncryptedBytes = RSA_public_encrypt(Source.Num(), Source.GetData(), Dest.GetData(), (RSA*)Key, RSA_PKCS1_PADDING);
	if (NumEncryptedBytes == -1)
	{
		Dest.Empty(0);
	}
	return NumEncryptedBytes;
}

int32 FEncryptionContextOpenSSL::EncryptPrivate_RSA(TArrayView<const uint8> Source, TArray<uint8>& Dest, FRSAKeyHandle Key)
{
	Dest.SetNum(GetKeySize_RSA(Key));
	int NumEncryptedBytes = RSA_private_encrypt(Source.Num(), Source.GetData(), Dest.GetData(), (RSA*)Key, RSA_PKCS1_PADDING);
	if (NumEncryptedBytes == -1)
	{
		Dest.Empty(0);
	}
	return NumEncryptedBytes;
}

int32 FEncryptionContextOpenSSL::DecryptPublic_RSA(TArrayView<const uint8> Source, TArray<uint8>& Dest, FRSAKeyHandle Key)
{
	Dest.SetNum(GetKeySize_RSA(Key) - RSA_PKCS1_PADDING_SIZE);
	int NumDecryptedBytes = RSA_public_decrypt(Source.Num(), Source.GetData(), Dest.GetData(), (RSA*)Key, RSA_PKCS1_PADDING);
	if (NumDecryptedBytes == -1)
	{
		Dest.Empty(0);
	}
	else
	{
		Dest.SetNum(NumDecryptedBytes);
	}
	return NumDecryptedBytes;
}

int32 FEncryptionContextOpenSSL::DecryptPrivate_RSA(TArrayView<const uint8> Source, TArray<uint8>& Dest, FRSAKeyHandle Key)
{
	Dest.SetNum(GetKeySize_RSA(Key) - RSA_PKCS1_PADDING_SIZE);
	int NumDecryptedBytes = RSA_private_decrypt(Source.Num(), Source.GetData(), Dest.GetData(), (RSA*)Key, RSA_PKCS1_PADDING);
	if (NumDecryptedBytes == -1)
	{
		Dest.Empty(0);
	}
	else
	{
		Dest.SetNum(NumDecryptedBytes);
	}
	return NumDecryptedBytes;
}

EPlatformCryptoResult FEncryptionContextOpenSSL::CreateRandomBytes(const TArrayView<uint8> OutData)
{
	return (RAND_bytes(OutData.GetData(), OutData.Num()) == 1) ? EPlatformCryptoResult::Success : EPlatformCryptoResult::Failure;
}

EPlatformCryptoResult FEncryptionContextOpenSSL::CreatePseudoRandomBytes(const TArrayView<uint8> OutData)
{
#if OPENSSL_VERSION_NUMBER < 0x10100000L
	return (RAND_pseudo_bytes(OutData.GetData(), OutData.Num()) == 1) ? EPlatformCryptoResult::Success : EPlatformCryptoResult::Failure;
#else // OPENSSL_VERSION_NUMBER < 0x10100000L
	return CreateRandomBytes(OutData);
#endif // OPENSSL_VERSION_NUMBER < 0x10100000L
}

FSHA256Hasher FEncryptionContextOpenSSL::CreateSHA256Hasher()
{
	return FSHA256Hasher();
}

bool FEncryptionContextOpenSSL::CalcSHA256(const TArrayView<const uint8> Source, TArray<uint8>& OutHash)
{
	FSHA256Hasher Hasher = CreateSHA256Hasher();

	if (Hasher.Init() != EPlatformCryptoResult::Success)
	{
		UE_LOGF(LogPlatformCryptoOpenSSL, Verbose,
			"[FEncryptionContextOpenSSL::CalcSHA256] Failed to create SHA256Hasher.");

		return false;
	}

	if (Hasher.Update(Source) != EPlatformCryptoResult::Success)
	{
		UE_LOGF(LogPlatformCryptoOpenSSL, Verbose,
			"[FEncryptionContextOpenSSL::CalcSHA256] Failed to update SHA256Hasher.");

		return false;
	}

	OutHash.Empty();
	OutHash.AddUninitialized(FSHA256Hasher::OutputByteLength);

	if (Hasher.Finalize(OutHash) != EPlatformCryptoResult::Success)
	{
		UE_LOGF(LogPlatformCryptoOpenSSL, Verbose,
			"[FEncryptionContextOpenSSL::CalcSHA256] Failed to finalize SHA256Hasher.");

		return false;
	}

	return true;
}

#endif // PLATFORMCRYPTOCONTEXT_USE_OPENSSL
