// Copyright Epic Games, Inc. All Rights Reserved.

#if PLATFORMCRYPTOCONTEXT_USE_OPENSSL

#include "PlatformCryptoAesDecryptorsOpenSSL.h"
#include <openssl/opensslv.h>

#define OPENSSL_CIPHER_SUCCESS 1

// Backwards compatibility for older version of openssl than 1.1.0
#if OPENSSL_VERSION_NUMBER < 0x10100000L
#define EVP_CIPHER_CTX_reset(c) EVP_CIPHER_CTX_cleanup(c)
#endif // OPENSSL_VERSION_NUMBER < 0x10100000L

const TCHAR* LexToString(FPlatformCryptoDecryptor_AES_Base_OpenSSL::EDecryptorState DecryptorState)
{
	switch (DecryptorState)
	{
	case FPlatformCryptoDecryptor_AES_Base_OpenSSL::EDecryptorState::Uninitialized:
		return TEXT("Uninitialized");
	case FPlatformCryptoDecryptor_AES_Base_OpenSSL::EDecryptorState::Initialized:
		return TEXT("Initialized");
	case FPlatformCryptoDecryptor_AES_Base_OpenSSL::EDecryptorState::Finalized:
		return TEXT("Finalized");
	}

	checkNoEntry();
	return TEXT("");
}

EPlatformCryptoResult FPlatformCryptoDecryptor_AES_Base_OpenSSL::Initialize(const EVP_CIPHER* Cipher, const TArrayView<const uint8> Key, const TOptional<TArrayView<const uint8>> InitializationVector)
{
	if (State != EDecryptorState::Uninitialized)
	{
		UE_LOGF(LogPlatformCryptoOpenSSL, Warning, "FPlatformCryptoDecryptor_AES_Base_OpenSSL::Update: Invalid state. Was %ls, but should be Uninitialized", LexToString(State));
		return EPlatformCryptoResult::Failure;
	}

	const uint8* const KeyPtr = Key.GetData();
	const uint8* const InitializationVectorPtr = InitializationVector.IsSet() ? InitializationVector.GetValue().GetData() : nullptr;

	const int InitResult = EVP_DecryptInit_ex(EVPContext.Get(), Cipher, nullptr, KeyPtr, InitializationVectorPtr);
	if (InitResult != OPENSSL_CIPHER_SUCCESS)
	{
		UE_LOGF(LogPlatformCryptoOpenSSL, Warning, "FPlatformCryptoDecryptor_AES_Base_OpenSSL::Initialize: EVP_DecryptInit_ex failed. Result=[%d]", InitResult);
		return EPlatformCryptoResult::Failure;
	}

	State = EDecryptorState::Initialized;
	return EPlatformCryptoResult::Success;
}

int32 FPlatformCryptoDecryptor_AES_Base_OpenSSL::GetCipherBlockSizeBytes() const
{
	return EVP_CIPHER_CTX_block_size(EVPContext.Get());
}

int32 FPlatformCryptoDecryptor_AES_Base_OpenSSL::GetCipherInitializationVectorSizeBytes() const
{
	return EVP_CIPHER_CTX_iv_length(EVPContext.Get());
}

EPlatformCryptoResult FPlatformCryptoDecryptor_AES_Base_OpenSSL::SetAuthTag(const TArrayView<const uint8> AuthTag)
{
	if (State != EDecryptorState::Initialized)
	{
		UE_LOGF(LogPlatformCryptoOpenSSL, Warning, "FPlatformCryptoDecryptor_AES_Base_OpenSSL::SetAuthTag: Invalid state. Was %ls, but should be Initialized", LexToString(State));
		return EPlatformCryptoResult::Failure;
	}

	if (AuthTag.Num() != GetCipherAuthTagSizeBytes())
	{
		UE_LOGF(LogPlatformCryptoOpenSSL, Warning, "FPlatformCryptoDecryptor_AES_Base_OpenSSL::SetAuthTag: Invalid AuthTag Size. TagSize=[%d] Expected=[%d]", AuthTag.Num(), GetCipherAuthTagSizeBytes());
		return EPlatformCryptoResult::Failure;
	}

	const int SetTagResult = EVP_CIPHER_CTX_ctrl(EVPContext.Get(), EVP_CTRL_GCM_SET_TAG, AuthTag.Num(), const_cast<uint8*>(AuthTag.GetData()));
	if (SetTagResult != OPENSSL_CIPHER_SUCCESS)
	{
		UE_LOGF(LogPlatformCryptoOpenSSL, Warning, "FPlatformCryptoDecryptor_AES_Base_OpenSSL::SetAuthTag: EVP_CIPHER_CTX_ctrl failed. Result=[%d]", SetTagResult);
		return EPlatformCryptoResult::Failure;
	}

	return EPlatformCryptoResult::Success;
}

EPlatformCryptoResult FPlatformCryptoDecryptor_AES_Base_OpenSSL::Update(const TArrayView<const uint8> Ciphertext, const TArrayView<uint8> OutPlaintext, int32& OutPlaintextBytesWritten)
{
	OutPlaintextBytesWritten = 0;

	if (State != EDecryptorState::Initialized)
	{
		UE_LOGF(LogPlatformCryptoOpenSSL, Warning, "FPlatformCryptoDecryptor_AES_Base_OpenSSL::Update: Invalid state. Was %ls, but should be Initialized", LexToString(State));
		return EPlatformCryptoResult::Failure;
	}

	const int32 RequiredBufferSize = GetUpdateBufferSizeBytes(Ciphertext);
	if (OutPlaintext.Num() < RequiredBufferSize)
	{
		UE_LOGF(LogPlatformCryptoOpenSSL, Warning, "FPlatformCryptoDecryptor_AES_Base_OpenSSL::Update: Invalid buffer size. Was %d needed %d", OutPlaintext.Num(), RequiredBufferSize);
		return EPlatformCryptoResult::Failure;
	}

	const int UpdateResult = EVP_DecryptUpdate(EVPContext.Get(), OutPlaintext.GetData(), &OutPlaintextBytesWritten, Ciphertext.GetData(), Ciphertext.Num());
	if (UpdateResult != OPENSSL_CIPHER_SUCCESS)
	{
		UE_LOGF(LogPlatformCryptoOpenSSL, Warning, "FPlatformCryptoDecryptor_AES_Base_OpenSSL::Update: EVP_DecryptInit_ex failed. Result=[%d]", UpdateResult);
		return EPlatformCryptoResult::Failure;
	}

	return EPlatformCryptoResult::Success;
}

EPlatformCryptoResult FPlatformCryptoDecryptor_AES_Base_OpenSSL::Finalize(const TArrayView<uint8> OutPlaintext, int32& OutPlaintextBytesWritten)
{
	OutPlaintextBytesWritten = 0;

	if (State != EDecryptorState::Initialized)
	{
		UE_LOGF(LogPlatformCryptoOpenSSL, Warning, "FPlatformCryptoDecryptor_AES_Base_OpenSSL::Update: Invalid state. Was %ls, but should be Initialized", LexToString(State));
		return EPlatformCryptoResult::Failure;
	}

	if (OutPlaintext.Num() < GetFinalizeBufferSizeBytes())
	{
		return EPlatformCryptoResult::Failure;
	}

	const int FinalizeResult = EVP_DecryptFinal_ex(EVPContext.Get(), OutPlaintext.GetData(), &OutPlaintextBytesWritten);
	if (FinalizeResult != OPENSSL_CIPHER_SUCCESS)
	{
		UE_LOGF(LogPlatformCryptoOpenSSL, Warning, "FPlatformCryptoDecryptor_AES_Base_OpenSSL::Finalize: EVP_DecryptFinal_ex failed. Result=[%d]", FinalizeResult);
		return EPlatformCryptoResult::Failure;
	}

	return EPlatformCryptoResult::Success;
}

EPlatformCryptoResult FPlatformCryptoDecryptor_AES_Base_OpenSSL::Reset(const TArrayView<const uint8> InitializationVector)
{
	if (State != EDecryptorState::Initialized && State != EDecryptorState::Finalized)
	{
		UE_LOGF(LogPlatformCryptoOpenSSL, Warning, "FPlatformCryptoDecryptor_AES_Base_OpenSSL::Reset: Invalid state. Was %ls, but should be Initialized or Finalized", LexToString(State));
		return EPlatformCryptoResult::Failure;
	}

	const int32 IVExpectedLength = EVP_CIPHER_CTX_iv_length(EVPContext.Get());
	if (InitializationVector.Num() < IVExpectedLength)
	{
		UE_LOGF(LogPlatformCryptoOpenSSL, Warning, "FPlatformCryptoDecryptor_AES_Base_OpenSSL::Reset: Invalid InitializationVector Size. InitializationVectorSize=[%d] Expected=[%d]", InitializationVector.Num(), IVExpectedLength);
		return EPlatformCryptoResult::Failure;
	}

	const uint8* InitializationVectorPtr = InitializationVector.Num() == 0 ? nullptr : InitializationVector.GetData();
	const int InitResult = EVP_DecryptInit_ex(EVPContext.Get(), nullptr, nullptr, nullptr, InitializationVectorPtr);
	if (InitResult != OPENSSL_CIPHER_SUCCESS)
	{
		UE_LOGF(LogPlatformCryptoOpenSSL, Warning, "FPlatformCryptoDecryptor_AES_Base_OpenSSL::Reset: EVP_DecryptInit_ex failed. Result=[%d]", InitResult);
		return EPlatformCryptoResult::Failure;
	}

	State = EDecryptorState::Initialized;
	return EPlatformCryptoResult::Success;
}

EPlatformCryptoResult FPlatformCryptoDecryptor_AES_Base_OpenSSL::Reset()
{
	const int ResetResult = EVP_CIPHER_CTX_reset(EVPContext.Get());
	if (ResetResult != OPENSSL_CIPHER_SUCCESS)
	{
		UE_LOGF(LogPlatformCryptoOpenSSL, Warning, "FPlatformCryptoDecryptor_AES_Base_OpenSSL::Reset: EVP_CIPHER_CTX_reset failed. Result=[%d]", ResetResult);
		return EPlatformCryptoResult::Failure;
	}

	State = EDecryptorState::Uninitialized;
	return EPlatformCryptoResult::Success;
}


/*static*/
TUniquePtr<IPlatformCryptoDecryptor> FPlatformCryptoDecryptor_AES_256_ECB_OpenSSL::Create(const TArrayView<const uint8> Key)
{
	const EVP_CIPHER* Cipher = EVP_aes_256_ecb();

	const int32 ExpectedKeyLength = EVP_CIPHER_key_length(Cipher);
	if (Key.Num() != ExpectedKeyLength)
	{
		UE_LOGF(LogPlatformCryptoOpenSSL, Warning, "Invalid Key Size, failed to create Decryptor. KeySize=[%d] Expected=[%d]", Key.Num(), ExpectedKeyLength);
		return nullptr;
	}

	TUniquePtr<FPlatformCryptoDecryptor_AES_256_ECB_OpenSSL> Decryptor(new FPlatformCryptoDecryptor_AES_256_ECB_OpenSSL());
	if (Decryptor->Initialize(Cipher, Key, TOptional<TArrayView<const uint8>>()) != EPlatformCryptoResult::Success)
	{
		return nullptr;
	}

	return Decryptor;
}

FName FPlatformCryptoDecryptor_AES_256_ECB_OpenSSL::GetCipherName() const
{
	return FName(TEXT("AES_256_ECB"));
}


int32 FPlatformCryptoDecryptor_AES_256_ECB_OpenSSL::GetCipherAuthTagSizeBytes() const
{
	return AES256_ECB_AuthTagSizeInBytes;
}

int32 FPlatformCryptoDecryptor_AES_256_ECB_OpenSSL::GetUpdateBufferSizeBytes(const TArrayView<const uint8> Ciphertext) const
{
	return Ciphertext.Num() + GetCipherBlockSizeBytes();
}

int32 FPlatformCryptoDecryptor_AES_256_ECB_OpenSSL::GetFinalizeBufferSizeBytes() const
{
	return GetCipherBlockSizeBytes();
}

/*static*/
TUniquePtr<IPlatformCryptoDecryptor> FPlatformCryptoDecryptor_AES_256_CBC_OpenSSL::Create(const TArrayView<const uint8> Key, const TArrayView<const uint8> InitializationVector)
{
	const EVP_CIPHER* Cipher = EVP_aes_256_cbc();

	const int32 ExpectedKeyLength = EVP_CIPHER_key_length(Cipher);
	if (Key.Num() != ExpectedKeyLength)
	{
		UE_LOGF(LogPlatformCryptoOpenSSL, Warning, "Invalid Key Size, failed to create Decryptor. KeySize=[%d] Expected=[%d]", Key.Num(), ExpectedKeyLength);
		return nullptr;
	}

	const int32 IVExpectedLength = EVP_CIPHER_iv_length(Cipher);
	if (InitializationVector.Num() < IVExpectedLength)
	{
		UE_LOGF(LogPlatformCryptoOpenSSL, Warning, "Invalid InitializationVector Size, failed to create Decryptor. InitializationVectorSize=[%d] Expected=[%d]", InitializationVector.Num(), IVExpectedLength);
		return nullptr;
	}

	TUniquePtr<FPlatformCryptoDecryptor_AES_256_CBC_OpenSSL> Decryptor(new FPlatformCryptoDecryptor_AES_256_CBC_OpenSSL());
	if (Decryptor->Initialize(EVP_aes_256_cbc(), Key, InitializationVector) != EPlatformCryptoResult::Success)
	{
		return nullptr;
	}

	return Decryptor;
}

FName FPlatformCryptoDecryptor_AES_256_CBC_OpenSSL::GetCipherName() const
{
	return FName(TEXT("AES_256_CBC"));
}

int32 FPlatformCryptoDecryptor_AES_256_CBC_OpenSSL::GetCipherAuthTagSizeBytes() const
{
	return AES256_CBC_AuthTagSizeInBytes;
}


int32 FPlatformCryptoDecryptor_AES_256_CBC_OpenSSL::GetUpdateBufferSizeBytes(const TArrayView<const uint8> Ciphertext) const
{
	return Ciphertext.Num() + GetCipherBlockSizeBytes();
}

int32 FPlatformCryptoDecryptor_AES_256_CBC_OpenSSL::GetFinalizeBufferSizeBytes() const
{
	return GetCipherBlockSizeBytes();
}

/*static*/
TUniquePtr<IPlatformCryptoDecryptor> FPlatformCryptoDecryptor_AES_256_GCM_OpenSSL::Create(const TArrayView<const uint8> Key, const TArrayView<const uint8> InitializationVector, const TArrayView<const uint8> AuthTag)
{
	const EVP_CIPHER* Cipher = EVP_aes_256_gcm();

	const int32 ExpectedKeyLength = EVP_CIPHER_key_length(Cipher);
	if (Key.Num() != ExpectedKeyLength)
	{
		UE_LOGF(LogPlatformCryptoOpenSSL, Warning, "Invalid Key Size, failed to create Decryptor. KeySize=[%d] Expected=[%d]", Key.Num(), ExpectedKeyLength);
		return nullptr;
	}

	const int32 IVExpectedLength = EVP_CIPHER_iv_length(Cipher);
	if (InitializationVector.Num() < IVExpectedLength)
	{
		UE_LOGF(LogPlatformCryptoOpenSSL, Warning, "Invalid InitializationVector Size, failed to create Decryptor. InitializationVectorSize=[%d] Expected=[%d]", InitializationVector.Num(), IVExpectedLength);
		return nullptr;
	}

	if (AuthTag.Num() < AES256_GCM_AuthTagSizeInBytes)
	{

		return nullptr;
	}

	TUniquePtr<FPlatformCryptoDecryptor_AES_256_GCM_OpenSSL> Decryptor(new FPlatformCryptoDecryptor_AES_256_GCM_OpenSSL());
	if (Decryptor->Initialize(Cipher, Key, InitializationVector) != EPlatformCryptoResult::Success)
	{
		return nullptr;
	}

	if (Decryptor->SetAuthTag(AuthTag) != EPlatformCryptoResult::Success)
	{
		return nullptr;
	}

	return Decryptor;
}

FName FPlatformCryptoDecryptor_AES_256_GCM_OpenSSL::GetCipherName() const
{
	return FName(TEXT("AES_256_GCM"));
}

int32 FPlatformCryptoDecryptor_AES_256_GCM_OpenSSL::GetCipherAuthTagSizeBytes() const
{
	return AES256_GCM_AuthTagSizeInBytes;
}

int32 FPlatformCryptoDecryptor_AES_256_GCM_OpenSSL::GetUpdateBufferSizeBytes(const TArrayView<const uint8> Ciphertext) const
{
	// GCM always decrypts to the same amount of plaintext bytes as input ciphertext
	return Ciphertext.Num();
}

int32 FPlatformCryptoDecryptor_AES_256_GCM_OpenSSL::GetFinalizeBufferSizeBytes() const
{
	// GCM finalization will always output 0 bytes, it only finalizes authentication part
	return 0;
}

// Undef what we defined above
#if OPENSSL_VERSION_NUMBER < 0x10100000L
#undef EVP_CIPHER_CTX_reset
#endif // OPENSSL_VERSION_NUMBER < 0x10100000L

#undef OPENSSL_CIPHER_SUCCESS

#endif // PLATFORMCRYPTOCONTEXT_USE_OPENSSL

