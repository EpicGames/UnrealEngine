// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlatformCryptoTypes.h"
#include "PlatformCryptoOpenSSLTypes.h"

class FPlatformCryptoDecryptor_AES_Base_OpenSSL
	: public IPlatformCryptoDecryptor
{
public:
	virtual ~FPlatformCryptoDecryptor_AES_Base_OpenSSL() = default;

	enum class EDecryptorState
	{
		/** Initialize needs to be called before data may be decrypted successfully */
		Uninitialized,
		/** Initialize succeeded and is currently able to decrypt data */
		Initialized,
		/** Finalize has been called, and Initialize must be called before it may be re-used */
		Finalized
	};

	/**
	 * Initialize our decryptor with a specific Cipher, Key, and (optionally) an Initialization Vector.
	 *
	 * @param Cipher OpenSSL Cipher to use
	 * @param Key Key to use.  Must be the correct amount of bits for the Cipher value
	 * @param InitializationVector Optional Initialization Vector if the Cipher requires one. Must be the correct amount of bits for the Cipher value if set
	 */
	EPlatformCryptoResult Initialize(const EVP_CIPHER* Cipher, const TArrayView<const uint8> Key, const TOptional<TArrayView<const uint8>> InitializationVector = TOptional<TArrayView<const uint8>>());

	//~ Begin IPlatformCryptoDecryptor Interface
	virtual EPlatformCryptoResult SetAuthTag(const TArrayView<const uint8> AuthTag) override final;
	virtual int32 GetCipherBlockSizeBytes() const override final;
	virtual int32 GetCipherInitializationVectorSizeBytes() const override final;
	virtual EPlatformCryptoResult Update(const TArrayView<const uint8> Ciphertext, const TArrayView<uint8> OutPlaintext, int32& OutPlaintextBytesWritten) override final;
	virtual EPlatformCryptoResult Finalize(const TArrayView<uint8> OutPlaintext, int32& OutPlaintextBytesWritten) override final;
	virtual EPlatformCryptoResult Reset(const TArrayView<const uint8> InitializationVector) override final;
	//~ End IPlatformCryptoDecryptor Interface

	/**
	 * Resets our context, requiring Initialize to be called again before it may be used.
	 *
	 * @return Success if our context was successfully reset, Failure otherwise
	 */
	EPlatformCryptoResult Reset();

protected:
	FPlatformCryptoDecryptor_AES_Base_OpenSSL() = default;

protected:
	FScopedEVPContext EVPContext;
	EDecryptorState State = EDecryptorState::Uninitialized;
};

class FPlatformCryptoDecryptor_AES_256_ECB_OpenSSL
	: public FPlatformCryptoDecryptor_AES_Base_OpenSSL
{
public:
	virtual ~FPlatformCryptoDecryptor_AES_256_ECB_OpenSSL() = default;

	static TUniquePtr<IPlatformCryptoDecryptor> Create(const TArrayView<const uint8> Key);

	//~ Begin IPlatformCryptoDecryptor Interface
	virtual FName GetCipherName() const override final;
	virtual int32 GetCipherAuthTagSizeBytes() const override final;
	virtual int32 GetUpdateBufferSizeBytes(const TArrayView<const uint8> Ciphertext) const override final;
	virtual int32 GetFinalizeBufferSizeBytes() const override final;
	//~ End IPlatformCryptoDecryptor Interface

private:
	FPlatformCryptoDecryptor_AES_256_ECB_OpenSSL() = default;
};

class FPlatformCryptoDecryptor_AES_256_CBC_OpenSSL
	: public FPlatformCryptoDecryptor_AES_Base_OpenSSL
{
public:
	virtual ~FPlatformCryptoDecryptor_AES_256_CBC_OpenSSL() = default;

	static TUniquePtr<IPlatformCryptoDecryptor> Create(const TArrayView<const uint8> Key, const TArrayView<const uint8> InitializationVector);

	//~ Begin IPlatformCryptoDecryptor Interface
	virtual FName GetCipherName() const override final;
	virtual int32 GetCipherAuthTagSizeBytes() const override final;
	virtual int32 GetUpdateBufferSizeBytes(const TArrayView<const uint8> Ciphertext) const override final;
	virtual int32 GetFinalizeBufferSizeBytes() const override final;
	//~ End IPlatformCryptoDecryptor Interface

private:
	FPlatformCryptoDecryptor_AES_256_CBC_OpenSSL() = default;
};

class FPlatformCryptoDecryptor_AES_256_GCM_OpenSSL
	: public FPlatformCryptoDecryptor_AES_Base_OpenSSL
{
public:
	virtual ~FPlatformCryptoDecryptor_AES_256_GCM_OpenSSL() = default;

	static TUniquePtr<IPlatformCryptoDecryptor> Create(const TArrayView<const uint8> Key, const TArrayView<const uint8> InitializationVector, const TArrayView<const uint8> AuthTag);

	//~ Begin IPlatformCryptoDecryptor Interface
	virtual FName GetCipherName() const override final;
	virtual int32 GetCipherAuthTagSizeBytes() const override final;
	virtual int32 GetUpdateBufferSizeBytes(const TArrayView<const uint8> Ciphertext) const override final;
	virtual int32 GetFinalizeBufferSizeBytes() const override final;
	//~ End IPlatformCryptoDecryptor Interface

private:
	FPlatformCryptoDecryptor_AES_256_GCM_OpenSSL() = default;
};
