// Copyright Epic Games, Inc. All Rights Reserved.

#if PLATFORMCRYPTOCONTEXT_USE_OPENSSL

#include "PlatformCryptoAesEncryptorsOpenSSL.h"
#include <openssl/opensslv.h>

#define OPENSSL_CIPHER_SUCCESS 1

// Backwards compatibility for older version of openssl than 1.1.0
#if OPENSSL_VERSION_NUMBER < 0x10100000L
#define EVP_CIPHER_CTX_reset(c) EVP_CIPHER_CTX_cleanup(c)
#endif // OPENSSL_VERSION_NUMBER < 0x10100000L

const TCHAR* LexToString(FPlatformCryptoEncryptor_AES_Base_OpenSSL::EEncryptorState EncryptorState)
{
	switch (EncryptorState)
	{
	case FPlatformCryptoEncryptor_AES_Base_OpenSSL::EEncryptorState::Uninitialized:
		return TEXT("Uninitialized");
	case FPlatformCryptoEncryptor_AES_Base_OpenSSL::EEncryptorState::Initialized:
		return TEXT("Initialized");
	case FPlatformCryptoEncryptor_AES_Base_OpenSSL::EEncryptorState::Finalized:
		return TEXT("Finalized");
	}

	checkNoEntry();
	return TEXT("");
}

EPlatformCryptoResult FPlatformCryptoEncryptor_AES_Base_OpenSSL::Initialize(const EVP_CIPHER* Cipher, const TArrayView<const uint8> Key, const TOptional<TArrayView<const uint8>> InitializationVector)
{
	if (State != EEncryptorState::Uninitialized)
	{
		UE_LOGF(LogPlatformCryptoOpenSSL, Warning, "FPlatformCryptoEncryptor_AES_Base_OpenSSL::Update: Invalid state. Was %ls, but should be Uninitialized", LexToString(State));
		return EPlatformCryptoResult::Failure;
	}

	const uint8* const KeyPtr = Key.GetData();
	const uint8* const InitializationVectorPtr = InitializationVector.IsSet() ? InitializationVector.GetValue().GetData() : nullptr;

	const int InitResult = EVP_EncryptInit_ex(EVPContext.Get(), Cipher, nullptr, KeyPtr, InitializationVectorPtr);
	if (InitResult != OPENSSL_CIPHER_SUCCESS)
	{
		UE_LOGF(LogPlatformCryptoOpenSSL, Warning, "FPlatformCryptoEncryptor_AES_Base_OpenSSL::Initialize: EVP_EncryptInit_ex failed. Result=[%d]", InitResult);
		return EPlatformCryptoResult::Failure;
	}

	State = EEncryptorState::Initialized;
	return EPlatformCryptoResult::Success;
}

int32 FPlatformCryptoEncryptor_AES_Base_OpenSSL::GetCipherBlockSizeBytes() const
{
	return EVP_CIPHER_CTX_block_size(EVPContext.Get());
}

int32 FPlatformCryptoEncryptor_AES_Base_OpenSSL::GetCipherInitializationVectorSizeBytes() const
{
	return EVP_CIPHER_CTX_iv_length(EVPContext.Get());
}

EPlatformCryptoResult FPlatformCryptoEncryptor_AES_Base_OpenSSL::GenerateAuthTag(const TArrayView<uint8> OutAuthTag, int32& OutAuthTagBytesWritten) const
{
	OutAuthTagBytesWritten = 0;

	if (State != EEncryptorState::Finalized)
	{
		UE_LOGF(LogPlatformCryptoOpenSSL, Warning, "FPlatformCryptoEncryptor_AES_Base_OpenSSL::GenerateAuthTag: Invalid state. Was %ls, but should be Finalized", LexToString(State));
		return EPlatformCryptoResult::Failure;
	}

	const int32 AuthTagSizeBytes = GetCipherAuthTagSizeBytes();

	if (OutAuthTag.Num() < AuthTagSizeBytes)
	{
		UE_LOGF(LogPlatformCryptoOpenSSL, Warning, "FPlatformCryptoDecryptor_AES_Base_OpenSSL::GenerateAuthTag: Invalid AuthTag Size. TagSize=[%d] Expected=[%d]", OutAuthTag.Num(), AuthTagSizeBytes);
		return EPlatformCryptoResult::Failure;
	}

	const int GenerateAuthTagResult = EVP_CIPHER_CTX_ctrl(EVPContext.Get(), EVP_CTRL_GCM_GET_TAG, OutAuthTag.Num(), OutAuthTag.GetData());
	if (GenerateAuthTagResult != OPENSSL_CIPHER_SUCCESS)
	{
		UE_LOGF(LogPlatformCryptoOpenSSL, Warning, "FPlatformCryptoDecryptor_AES_Base_OpenSSL::GenerateAuthTag: EVP_CIPHER_CTX_ctrl failed. Result=[%d]", GenerateAuthTagResult);
		return EPlatformCryptoResult::Failure;
	}

	OutAuthTagBytesWritten = AuthTagSizeBytes;

	return EPlatformCryptoResult::Success;
}

EPlatformCryptoResult FPlatformCryptoEncryptor_AES_Base_OpenSSL::Update(const TArrayView<const uint8> Plaintext, const TArrayView<uint8> OutCiphertext, int32& OutCiphertextBytesWritten)
{
	OutCiphertextBytesWritten = 0;

	if (State != EEncryptorState::Initialized)
	{
		UE_LOGF(LogPlatformCryptoOpenSSL, Warning, "FPlatformCryptoEncryptor_AES_Base_OpenSSL::Update: Invalid state. Was %ls, but should be Initialized", LexToString(State));
		return EPlatformCryptoResult::Failure;
	}

	const int32 RequiredBufferSize = GetUpdateBufferSizeBytes(Plaintext);
	if (OutCiphertext.Num() < RequiredBufferSize)
	{
		UE_LOGF(LogPlatformCryptoOpenSSL, Warning, "FPlatformCryptoEncryptor_AES_Base_OpenSSL::Update: Invalid buffer size. Was %d needed %d", OutCiphertext.Num() ,RequiredBufferSize);
		return EPlatformCryptoResult::Failure;
	}

	const int UpdateResult = EVP_EncryptUpdate(EVPContext.Get(), OutCiphertext.GetData(), &OutCiphertextBytesWritten, Plaintext.GetData(), Plaintext.Num());
	if (UpdateResult != OPENSSL_CIPHER_SUCCESS)
	{
		UE_LOGF(LogPlatformCryptoOpenSSL, Warning, "FPlatformCryptoEncryptor_AES_Base_OpenSSL::Update: EVP_EncryptInit_ex failed. Result=[%d]", UpdateResult);
		return EPlatformCryptoResult::Failure;
	}

	return EPlatformCryptoResult::Success;
}

EPlatformCryptoResult FPlatformCryptoEncryptor_AES_Base_OpenSSL::Finalize(const TArrayView<uint8> OutCiphertext, int32& OutCiphertextBytesWritten)
{
	OutCiphertextBytesWritten = 0;

	if (State != EEncryptorState::Initialized)
	{
		UE_LOGF(LogPlatformCryptoOpenSSL, Warning, "FPlatformCryptoEncryptor_AES_Base_OpenSSL::Update: Invalid state. Was %ls, but should be Initialized", LexToString(State));
		return EPlatformCryptoResult::Failure;
	}

	if (OutCiphertext.Num() < GetFinalizeBufferSizeBytes())
	{
		return EPlatformCryptoResult::Failure;
	}

	const int FinalizeResult = EVP_EncryptFinal_ex(EVPContext.Get(), OutCiphertext.GetData(), &OutCiphertextBytesWritten);
	if (FinalizeResult != OPENSSL_CIPHER_SUCCESS)
	{
		UE_LOGF(LogPlatformCryptoOpenSSL, Warning, "FPlatformCryptoEncryptor_AES_Base_OpenSSL::Finalize: EVP_EncryptFinal_ex failed. Result=[%d]", FinalizeResult);
		return EPlatformCryptoResult::Failure;
	}

	State = EEncryptorState::Finalized;
	return EPlatformCryptoResult::Success;
}

EPlatformCryptoResult FPlatformCryptoEncryptor_AES_Base_OpenSSL::Reset(const TArrayView<const uint8> InitializationVector)
{
	if (State != EEncryptorState::Initialized && State != EEncryptorState::Finalized)
	{
		UE_LOGF(LogPlatformCryptoOpenSSL, Warning, "FPlatformCryptoEncryptor_AES_Base_OpenSSL::Reset: Invalid state. Was %ls, but should be Initialized or Finalized", LexToString(State));
		return EPlatformCryptoResult::Failure;
	}

	const int32 IVExpectedLength = EVP_CIPHER_CTX_iv_length(EVPContext.Get());
	if (InitializationVector.Num() < IVExpectedLength)
	{
		UE_LOGF(LogPlatformCryptoOpenSSL, Warning, "FPlatformCryptoDecryptor_AES_Base_OpenSSL::Reset: Invalid InitializationVector Size. InitializationVectorSize=[%d] Expected=[%d]", InitializationVector.Num(), IVExpectedLength);
		return EPlatformCryptoResult::Failure;
	}

	const uint8* InitializationVectorPtr = InitializationVector.Num() == 0 ? nullptr : InitializationVector.GetData();
	const int InitResult = EVP_EncryptInit_ex(EVPContext.Get(), nullptr, nullptr, nullptr, InitializationVectorPtr);
	if (InitResult != OPENSSL_CIPHER_SUCCESS)
	{
		UE_LOGF(LogPlatformCryptoOpenSSL, Warning, "FPlatformCryptoEncryptor_AES_Base_OpenSSL::Reset: EVP_EncryptInit_ex failed. Result=[%d]", InitResult);
		return EPlatformCryptoResult::Failure;
	}

	State = EEncryptorState::Initialized;
	return EPlatformCryptoResult::Success;
}

EPlatformCryptoResult FPlatformCryptoEncryptor_AES_Base_OpenSSL::Reset()
{
	const int ResetResult = EVP_CIPHER_CTX_reset(EVPContext.Get());
	if (ResetResult != OPENSSL_CIPHER_SUCCESS)
	{
		UE_LOGF(LogPlatformCryptoOpenSSL, Warning, "FPlatformCryptoEncryptor_AES_Base_OpenSSL::Reset: EVP_CIPHER_CTX_reset failed. Result=[%d]", ResetResult);
		return EPlatformCryptoResult::Failure;
	}

	State = EEncryptorState::Uninitialized;
	return EPlatformCryptoResult::Success;
}


/*static*/
TUniquePtr<IPlatformCryptoEncryptor> FPlatformCryptoEncryptor_AES_256_ECB_OpenSSL::Create(const TArrayView<const uint8> Key)
{
	const EVP_CIPHER* Cipher = EVP_aes_256_ecb();

	const int32 ExpectedKeyLength = EVP_CIPHER_key_length(Cipher);
	if (Key.Num() != ExpectedKeyLength)
	{
		UE_LOGF(LogPlatformCryptoOpenSSL, Warning, "Invalid Key Size, failed to create Encryptor. KeySize=[%d] Expected=[%d]", Key.Num(), ExpectedKeyLength);
		return nullptr;
	}

	TUniquePtr<FPlatformCryptoEncryptor_AES_256_ECB_OpenSSL> Encryptor(new FPlatformCryptoEncryptor_AES_256_ECB_OpenSSL());
	if (Encryptor->Initialize(Cipher, Key, TOptional<TArrayView<const uint8>>()) != EPlatformCryptoResult::Success)
	{
		return nullptr;
	}

	return Encryptor;
}

FName FPlatformCryptoEncryptor_AES_256_ECB_OpenSSL::GetCipherName() const
{
	return FName(TEXT("AES_256_ECB"));
}


int32 FPlatformCryptoEncryptor_AES_256_ECB_OpenSSL::GetUpdateBufferSizeBytes(const TArrayView<const uint8> Plaintext) const
{
	return Plaintext.Num() + GetCipherBlockSizeBytes() - 1;
}

int32 FPlatformCryptoEncryptor_AES_256_ECB_OpenSSL::GetFinalizeBufferSizeBytes() const
{
	return GetCipherBlockSizeBytes();
}

int32 FPlatformCryptoEncryptor_AES_256_ECB_OpenSSL::GetCipherAuthTagSizeBytes() const
{
	return AES256_ECB_AuthTagSizeInBytes;
}


/*static*/
TUniquePtr<IPlatformCryptoEncryptor> FPlatformCryptoEncryptor_AES_256_CBC_OpenSSL::Create(const TArrayView<const uint8> Key, const TArrayView<const uint8> InitializationVector)
{
	const EVP_CIPHER* Cipher = EVP_aes_256_cbc();

	const int32 ExpectedKeyLength = EVP_CIPHER_key_length(Cipher);
	if (Key.Num() != ExpectedKeyLength)
	{
		UE_LOGF(LogPlatformCryptoOpenSSL, Warning, "Invalid Key Size, failed to create Encryptor. KeySize=[%d] Expected=[%d]", Key.Num(), ExpectedKeyLength);
		return nullptr;
	}

	const int32 IVExpectedLength = EVP_CIPHER_iv_length(Cipher);
	if (InitializationVector.Num() < IVExpectedLength)
	{
		UE_LOGF(LogPlatformCryptoOpenSSL, Warning, "Invalid InitializationVector Size, failed to create Encryptor. InitializationVectorSize=[%d] Expected=[%d]", InitializationVector.Num(), IVExpectedLength);
		return nullptr;
	}

	TUniquePtr<FPlatformCryptoEncryptor_AES_256_CBC_OpenSSL> Encryptor(new FPlatformCryptoEncryptor_AES_256_CBC_OpenSSL());
	if (Encryptor->Initialize(Cipher, Key, InitializationVector) != EPlatformCryptoResult::Success)
	{
		return nullptr;
	}

	return Encryptor;
}

FName FPlatformCryptoEncryptor_AES_256_CBC_OpenSSL::GetCipherName() const
{
	return FName(TEXT("AES_256_CBC"));
}

int32 FPlatformCryptoEncryptor_AES_256_CBC_OpenSSL::GetUpdateBufferSizeBytes(const TArrayView<const uint8> Plaintext) const
{
	return Plaintext.Num() + GetCipherBlockSizeBytes() - 1;
}

int32 FPlatformCryptoEncryptor_AES_256_CBC_OpenSSL::GetFinalizeBufferSizeBytes() const
{
	return GetCipherBlockSizeBytes();
}

int32 FPlatformCryptoEncryptor_AES_256_CBC_OpenSSL::GetCipherAuthTagSizeBytes() const
{
	return 0;
}


/*static*/
TUniquePtr<IPlatformCryptoEncryptor> FPlatformCryptoEncryptor_AES_256_GCM_OpenSSL::Create(const TArrayView<const uint8> Key, const TArrayView<const uint8> InitializationVector)
{
	const EVP_CIPHER* Cipher = EVP_aes_256_gcm();

	const int32 ExpectedKeyLength = EVP_CIPHER_key_length(Cipher);
	if (Key.Num() != ExpectedKeyLength)
	{
		UE_LOGF(LogPlatformCryptoOpenSSL, Warning, "Invalid Key Size, failed to create Encryptor. KeySize=[%d] Expected=[%d]", Key.Num(), ExpectedKeyLength);
		return nullptr;
	}

	const int32 IVExpectedLength = EVP_CIPHER_iv_length(Cipher);
	if (InitializationVector.Num() < IVExpectedLength)
	{
		UE_LOGF(LogPlatformCryptoOpenSSL, Warning, "Invalid InitializationVector Size, failed to create Decryptor. InitializationVectorSize=[%d] Expected=[%d]", InitializationVector.Num(), IVExpectedLength);
		return nullptr;
	}

	TUniquePtr<FPlatformCryptoEncryptor_AES_256_GCM_OpenSSL> Encryptor(new FPlatformCryptoEncryptor_AES_256_GCM_OpenSSL());
	if (Encryptor->Initialize(Cipher, Key, InitializationVector) != EPlatformCryptoResult::Success)
	{
		return nullptr;
	}

	return Encryptor;
}

FName FPlatformCryptoEncryptor_AES_256_GCM_OpenSSL::GetCipherName() const
{
	return FName(TEXT("AES_256_GCM"));
}

int32 FPlatformCryptoEncryptor_AES_256_GCM_OpenSSL::GetCipherAuthTagSizeBytes() const
{
	return AES256_GCM_AuthTagSizeInBytes;
}

int32 FPlatformCryptoEncryptor_AES_256_GCM_OpenSSL::GetUpdateBufferSizeBytes(const TArrayView<const uint8> Plaintext) const
{
	// GCM always encrypts to the same amount of ciphertext bytes as input plaintext
	return Plaintext.Num();
}

int32 FPlatformCryptoEncryptor_AES_256_GCM_OpenSSL::GetFinalizeBufferSizeBytes() const
{
	// GCM finalization will always output 0 bytes, it only finalizes authentication part
	return 0;
}

// Undef what we defined above
#if OPENSSL_VERSION_NUMBER < 0x10100000L
#undef EVP_CIPHER_CTX_reset
#endif // OPENSSL_VERSION_NUMBER < 0x10100000L

#undef OPENSSL_CIPHER_SUCCESS

#endif // PLATFORMCRYPTOCONTEXT_USE_OPENSSL
