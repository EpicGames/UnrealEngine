// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlatformCryptoTypes.h"
#include "PlatformCryptoOpenSSLTypes.h"

template <typename OptionalType> struct TOptional;

class FPlatformCryptoEncryptor_AES_Base_OpenSSL
	: public IPlatformCryptoEncryptor
{
public:
	virtual ~FPlatformCryptoEncryptor_AES_Base_OpenSSL() = default;

	enum class EEncryptorState
	{
		/** Initialize needs to be called before data may be encrypted successfully */
		Uninitialized,
		/** Initialize succeeded and is currently able to encrypt data */
		Initialized,
		/** Finalize has been called, and Initialize must be called before it may be re-used */
		Finalized
	};

	/**
	 * Initialize our encryptor with a specific Cipher, Key, and (optionally) an Initialization Vector.
	 *
	 * @param Cipher OpenSSL Cipher to use
	 * @param Key Key to use.  Must be the correct amount of bits for the Cipher value
	 * @param InitializationVector Optional Initialization Vector if the Cipher requires one. Must be the correct amount of bits for the Cipher value if set
	 */
	EPlatformCryptoResult Initialize(const EVP_CIPHER* Cipher, const TArrayView<const uint8> Key, const TOptional<TArrayView<const uint8>> InitializationVector);

	//~ Begin IPlatformCryptoEncryptor Interface
	virtual int32 GetCipherBlockSizeBytes() const override final;
	virtual int32 GetCipherInitializationVectorSizeBytes() const override final;
	virtual EPlatformCryptoResult GenerateAuthTag(const TArrayView<uint8> OutAuthTag, int32& OutAuthTagBytesWritten) const override final;
	virtual EPlatformCryptoResult Update(const TArrayView<const uint8> Plaintext, const TArrayView<uint8> OutCiphertext, int32& OutCiphertextBytesWritten) override final;
	virtual EPlatformCryptoResult Finalize(const TArrayView<uint8> OutCiphertext, int32& OutCiphertextBytesWritten) override final;
	virtual EPlatformCryptoResult Reset(const TArrayView<const uint8> InitializationVector) override final;
	//~ End IPlatformCryptoEncryptor Interface

	/**
	 * Resets our context, requiring Initialize to be called again before it may be used.
	 *
	 * @return Success if our context was successfully reset, Failure otherwise
	 */
	EPlatformCryptoResult Reset();

protected:
	FPlatformCryptoEncryptor_AES_Base_OpenSSL() = default;

protected:
	FScopedEVPContext EVPContext;
	EEncryptorState State = EEncryptorState::Uninitialized;
};

class FPlatformCryptoEncryptor_AES_256_ECB_OpenSSL
	: public FPlatformCryptoEncryptor_AES_Base_OpenSSL
{
public:
	virtual ~FPlatformCryptoEncryptor_AES_256_ECB_OpenSSL() = default;

	static TUniquePtr<IPlatformCryptoEncryptor> Create(const TArrayView<const uint8> Key);

	//~ Begin IPlatformCryptoEncryptor Interface
	virtual FName GetCipherName() const override final;
	virtual int32 GetCipherAuthTagSizeBytes() const override final;
	virtual int32 GetUpdateBufferSizeBytes(const TArrayView<const uint8> Plaintext) const override final;
	virtual int32 GetFinalizeBufferSizeBytes() const override final;
	//~ End IPlatformCryptoEncryptor Interface

private:
	FPlatformCryptoEncryptor_AES_256_ECB_OpenSSL() = default;
};

class FPlatformCryptoEncryptor_AES_256_CBC_OpenSSL
	: public FPlatformCryptoEncryptor_AES_Base_OpenSSL
{
public:
	virtual ~FPlatformCryptoEncryptor_AES_256_CBC_OpenSSL() = default;

	static TUniquePtr<IPlatformCryptoEncryptor> Create(const TArrayView<const uint8> Key, const TArrayView<const uint8> InitializationVector);

	//~ Begin IPlatformCryptoEncryptor Interface
	virtual FName GetCipherName() const override final;
	virtual int32 GetCipherAuthTagSizeBytes() const override final;
	virtual int32 GetUpdateBufferSizeBytes(const TArrayView<const uint8> Plaintext) const override final;
	virtual int32 GetFinalizeBufferSizeBytes() const override final;
	//~ End IPlatformCryptoEncryptor Interface

private:
	FPlatformCryptoEncryptor_AES_256_CBC_OpenSSL() = default;
};

class FPlatformCryptoEncryptor_AES_256_GCM_OpenSSL
	: public FPlatformCryptoEncryptor_AES_Base_OpenSSL
{
public:
	virtual ~FPlatformCryptoEncryptor_AES_256_GCM_OpenSSL() = default;

	static TUniquePtr<IPlatformCryptoEncryptor> Create(const TArrayView<const uint8> Key, const TArrayView<const uint8> InitializationVector);

	//~ Begin IPlatformCryptoEncryptor Interface
	virtual FName GetCipherName() const override final;
	virtual int32 GetCipherAuthTagSizeBytes() const override final;
	virtual int32 GetUpdateBufferSizeBytes(const TArrayView<const uint8> Plaintext) const override final;
	virtual int32 GetFinalizeBufferSizeBytes() const override final;
	//~ End IPlatformCryptoEncryptor Interface

private:
	FPlatformCryptoEncryptor_AES_256_GCM_OpenSSL() = default;
};
