// Copyright Epic Games, Inc. All Rights Reserved.

#include "IPlatformCryptoContext.h"
#include "PlatformCryptoContextIncludes.h"


IMPLEMENT_MODULE( IPlatformCryptoContext, PlatformCryptoContext )

void IPlatformCryptoContext::StartupModule()
{
	// This code will execute after your module is loaded into memory (but after global variables are initialized, of course.)
	FEncryptionContext::OnStartup(this);
}


void IPlatformCryptoContext::ShutdownModule()
{
	// This function may be called during shutdown to clean up your module.  For modules that support dynamic reloading,
	// we call this function before unloading the module.
	FEncryptionContext::OnShutdown(this);
}
