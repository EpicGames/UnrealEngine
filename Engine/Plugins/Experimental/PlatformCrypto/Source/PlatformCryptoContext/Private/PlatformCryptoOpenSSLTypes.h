// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EncryptionContextOpenSSL.h" // IWYU pragma: keep

THIRD_PARTY_INCLUDES_START
#include <openssl/evp.h>
#include <openssl/ossl_typ.h>
THIRD_PARTY_INCLUDES_END

DECLARE_LOG_CATEGORY_EXTERN(LogPlatformCryptoOpenSSL, Warning, All);

static constexpr const int32 AES256_ECB_AuthTagSizeInBytes = 0; // Not Supported
static constexpr const int32 AES256_CBC_AuthTagSizeInBytes = 0; // Not Supported
static constexpr const int32 AES256_GCM_AuthTagSizeInBytes = 16; // 128-bit

/**
 * RAII wrapper for the OpenSSL EVP Cipher context object
 *
 * The EVP cipher context stores state related to OpenSSL encryption/decryption functions, and requires initialization
 * by the user, by use of functions such as EVP_EncryptInit_ex.
 */
class FScopedEVPContext
{
public:
	/**
	 * Creates our OpenSSL Cipher Context on construction
	 */
	FScopedEVPContext() :
		Context(EVP_CIPHER_CTX_new())
	{
	}

	/**
	 * Free our OpenSSL Cipher Context
	 */
	~FScopedEVPContext()
	{
		EVP_CIPHER_CTX_free(Context);
	}

	/**
	 * Get our OpenSSL Cipher Context
	 */
	EVP_CIPHER_CTX* Get() const { return Context; }

private:
	/** Disable copying/assigning */
	FScopedEVPContext(const FScopedEVPContext& Other) = delete;
	FScopedEVPContext& operator=(const FScopedEVPContext& Other) = delete;

	EVP_CIPHER_CTX* Context;
};

/**
 * RAII wrapper for the OpenSSL EVP Message Digest context object
 *
 * The EVP message digest context stores state related to OpenSSL digest functions, and requires initialization
 * by the user, by use of functions such as EVP_DigestVerifyInit.
 */
class FScopedEVPMDContext
{
public:
	/**
	 * Creates our OpenSSL Message Digest Context on construction
	 */
	FScopedEVPMDContext() :
		Context(EVP_MD_CTX_create())
	{
	}

	/** Disable copying/assigning */
	FScopedEVPMDContext(FScopedEVPMDContext&) = delete;
	FScopedEVPMDContext& operator=(FScopedEVPMDContext&) = delete;

	/**
	 * Free our OpenSSL Message Digest Context
	 */
	~FScopedEVPMDContext()
	{
		EVP_MD_CTX_destroy(Context);
	}

	/**
	 * Get our OpenSSL Message Digest Context
	 */
	EVP_MD_CTX* Get() const { return Context; }

private:
	EVP_MD_CTX* Context;
};

/**
 * RAII wrapper for the OpenSSL EVP public key algorithm context object.
 */
class FScopedEvpPKeyContext
{
public:
	/**
	 * Creates OpenSSL EVP public key algorithm context on construction.
	 */
	FScopedEvpPKeyContext(int KeyAlgorithm) :
		Context(EVP_PKEY_CTX_new_id(KeyAlgorithm, nullptr))
	{
	}

	/** Disable copying/assigning */
	FScopedEvpPKeyContext(FScopedEvpPKeyContext&) = delete;
	FScopedEvpPKeyContext& operator=(FScopedEvpPKeyContext&) = delete;

	/**
	 * Releases OpenSSL EVP public key algorithm context.
	 */
	~FScopedEvpPKeyContext()
	{
		EVP_PKEY_CTX_free(Context);
	}

	/**
	 * Returns OpenSSL EVP public key algorithm context.
	 */
	EVP_PKEY_CTX* Get() const { return Context; }

private:
	EVP_PKEY_CTX* Context;
};