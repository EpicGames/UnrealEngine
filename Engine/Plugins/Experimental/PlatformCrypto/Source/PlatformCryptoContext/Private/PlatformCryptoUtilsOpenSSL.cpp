// Copyright Epic Games, Inc. All Rights Reserved.

#if PLATFORMCRYPTOCONTEXT_USE_OPENSSL

#include "PlatformCryptoUtilsOpenSSL.h"

void FPlatformCryptoUtilsOpenSSL::BigNumToArray(
	const BIGNUM* InNum, TArray<uint8>& OutBytes)
{
	int32 NumBytes = BN_num_bytes(InNum);
	OutBytes.SetNumZeroed(NumBytes);

	BN_bn2bin(InNum, OutBytes.GetData());
	Algo::Reverse(OutBytes);
}


void FPlatformCryptoUtilsOpenSSL::LoadBinaryIntoBigNum(
	const uint8* InData, int32 InDataSize, BIGNUM* InBigNum)
{
#if USE_LEGACY_OPENSSL
	TArray<uint8> Bytes(InData, InDataSize);
	Algo::Reverse(Bytes);
	BN_bin2bn(Bytes.GetData(), Bytes.Num(), InBigNum);
#else
	BN_lebin2bn(InData, InDataSize, InBigNum);
#endif
}

#endif // PLATFORMCRYPTOCONTEXT_USE_OPENSSL
