// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

THIRD_PARTY_INCLUDES_START
#include <openssl/bn.h>
THIRD_PARTY_INCLUDES_END


/**
 * Some platforms were upgraded to OpenSSL 1.1.1 while the others
 * were left on a previous version. There are some minor differences
 * we have to account for in the older version, so declare a handy
 * define that we can use to gate the code.
 */
#if !defined(OPENSSL_VERSION_NUMBER) || OPENSSL_VERSION_NUMBER < 0x10100000L
#define USE_LEGACY_OPENSSL 1
#else
#define USE_LEGACY_OPENSSL 0
#endif


class FPlatformCryptoUtilsOpenSSL
{

public:

	static void BigNumToArray(
		const BIGNUM* InNum, TArray<uint8>& OutBytes);

	static void LoadBinaryIntoBigNum(
		const uint8* InData, int32 InDataSize, BIGNUM* InBigNum);

};
