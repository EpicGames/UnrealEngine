// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Misc/IEngineCrypto.h"
#include "Templates/PimplPtr.h"

class IPlatformCryptoDecryptor;
class IPlatformCryptoEncryptor;
enum class EPlatformCryptoResult;
class IPlatformCryptoContext;
struct FCryptoKeyPair;

/** Implementation details for SHA256 computation using OpenSSL */
struct FSHA256HasherOpenSSL final
{
	FSHA256HasherOpenSSL(FSHA256HasherOpenSSL&&) = default;
	FSHA256HasherOpenSSL& operator=(FSHA256HasherOpenSSL&&) = default;

	/**
	 * Initialize the necessary state to begin computing a message digest.
	 * It is only necessary to call this function if the object is being used to compute multiple hashes, as the constructor will automatically call it the first time.
	 */
	PLATFORMCRYPTOCONTEXT_API EPlatformCryptoResult Init();

	/**
	 * Update the message digest computation with additional bytes
	 *
	 * @param InDataBuffer Buffer pointing to the data
	 */
	PLATFORMCRYPTOCONTEXT_API EPlatformCryptoResult Update(const TArrayView<const uint8> InDataBuffer);

	/**
	 * Finalize the computation of the message digest. After calling this method, Init must be called again if this object is meant to be reused for hashing a new input.
	 *
	 * @param OutDataBuffer A buffer that can hold the message digest bytes. Call GetOutputByteLength to determine the necessary size.
	 */
	PLATFORMCRYPTOCONTEXT_API EPlatformCryptoResult Finalize(const TArrayView<uint8> OutDataBuffer);

	/**
	 * The final message digest length in bytes.
	 */
	static constexpr const uint32 OutputByteLength = 32;

private:
	FSHA256HasherOpenSSL();
	FSHA256HasherOpenSSL(const FSHA256HasherOpenSSL&) = delete;
	FSHA256HasherOpenSSL& operator=(const FSHA256HasherOpenSSL&) = delete;
	friend class FEncryptionContextOpenSSL;

	struct FImplDetails;
	TPimplPtr<FImplDetails> Inner;
};
using FSHA256Hasher = FSHA256HasherOpenSSL;

/** Implementation details for ed25519 signing using OpenSSL */
class FSignerEd25519OpenSSL final
{
public:
	FSignerEd25519OpenSSL();

	FSignerEd25519OpenSSL(FSignerEd25519OpenSSL&&) = default;
	FSignerEd25519OpenSSL(const FSignerEd25519OpenSSL&) = delete;
	FSignerEd25519OpenSSL& operator=(FSignerEd25519OpenSSL&&) = default;
	FSignerEd25519OpenSSL& operator=(const FSignerEd25519OpenSSL&) = delete;

	~FSignerEd25519OpenSSL();

	/**
	* Set private key for signing
	*
	* @param Key Buffer pointing to the private key data
	* @return True if initialized succesfully
	*/
	bool SetPrivateKey(const TArrayView<const uint8> Key);

	/**
	* Signing data block
	*
	* @param Message Buffer pointing to the data
	* @param Signature Buffer pointing to the signature
	* @return True if signed succesfully
	*/
	bool Sign(const TArrayView<const uint8> Message, const TArrayView<uint8> Signature);
	
	/**
	* Check if signer are ready
	*/
	bool IsReady() const;

	/**
	 * The final signature length in bytes.
	*/
	static constexpr const uint32 OutputSignatureLength = 64;

	/**
	 * The key length in bytes.
	*/
	static constexpr const uint32 KeyBufferLength = 32;

private:
	friend class FEncryptionContextOpenSSL;

	struct FImplDetails;
	TPimplPtr<FImplDetails> Inner;
};
using FSignerEd25519 = FSignerEd25519OpenSSL;

/** Implementation details for ed25519 verify signature using OpenSSL */
class FVerifierEd25519OpenSSL final
{
public:
	FVerifierEd25519OpenSSL();

	FVerifierEd25519OpenSSL(FVerifierEd25519OpenSSL&&) = default;
	FVerifierEd25519OpenSSL(const FVerifierEd25519OpenSSL&) = delete;
	FVerifierEd25519OpenSSL& operator=(FVerifierEd25519OpenSSL&&) = default;
	FVerifierEd25519OpenSSL& operator=(const FVerifierEd25519OpenSSL&) = delete;

	~FVerifierEd25519OpenSSL();

	/**
	* Set public key for verify signature
	*
	* @param Key Buffer pointing to the public key data
	* @return True if inited successfully
	*/
	bool SetPublicKey(const TArrayView<const uint8> Key);

	/**
	* Verify data block
	*
	* @param Message Buffer pointing to the data
	* @param Signature Buffer pointing to the signature
	* @return True if signature are correct
	*/
	bool Verify(const TArrayView<const uint8> Message, const TArrayView<const uint8> Signature);

	/**
	* Check if verifier are ready
	*/
	bool IsReady() const;

	/**
	* The signature length in bytes.
	*/
	static constexpr const uint32 SignatureLength = 64;

	/**
	 * The key length in bytes.
	*/
	static constexpr const uint32 KeyBufferLength = 32;

private:
	friend class FEncryptionContextOpenSSL;

	struct FImplDetails;
	TPimplPtr<FImplDetails> Inner;
};
using FVerifierEd25519 = FVerifierEd25519OpenSSL;

/** Implementation of crypto key generator using OpenSSL. */
class FCryptoKeyGeneratorOpenSSL final
{
public:
	/**
	* Generates Ed25519 key pair.
	*
	* @return The structure with public/private key pair when success.
	*/
	TOptional<FCryptoKeyPair> GenerateEd25519() const;
};
using FCryptoKeyGenerator = FCryptoKeyGeneratorOpenSSL;

/**
 * Interface to certain cryptographic algorithms, using OpenSSL to implement them.
 */
class FEncryptionContextOpenSSL
{

public:

	static void OnStartup(IPlatformCryptoContext* const Context) {}
	static void OnShutdown(IPlatformCryptoContext* const Context) {}

	PLATFORMCRYPTOCONTEXT_API TArray<uint8> Encrypt_AES_256_ECB(const TArrayView<const uint8> Plaintext, const TArrayView<const uint8> Key, EPlatformCryptoResult& OutResult);
	PLATFORMCRYPTOCONTEXT_API TArray<uint8> Encrypt_AES_256_CBC(const TArrayView<const uint8> Plaintext, const TArrayView<const uint8> Key, const TArrayView<const uint8> InitializationVector, EPlatformCryptoResult& OutResult);
	PLATFORMCRYPTOCONTEXT_API TArray<uint8> Encrypt_AES_256_GCM(const TArrayView<const uint8> Plaintext, const TArrayView<const uint8> Key, const TArrayView<const uint8> Nonce, TArray<uint8>& OutAuthTag, EPlatformCryptoResult& OutResult);

	PLATFORMCRYPTOCONTEXT_API TArray<uint8> Decrypt_AES_256_ECB(const TArrayView<const uint8> Ciphertext, const TArrayView<const uint8> Key, EPlatformCryptoResult& OutResult);
	PLATFORMCRYPTOCONTEXT_API TArray<uint8> Decrypt_AES_256_CBC(const TArrayView<const uint8> Ciphertext, const TArrayView<const uint8> Key, const TArrayView<const uint8> InitializationVector, EPlatformCryptoResult& OutResult);
	PLATFORMCRYPTOCONTEXT_API TArray<uint8> Decrypt_AES_256_GCM(const TArrayView<const uint8> Ciphertext, const TArrayView<const uint8> Key, const TArrayView<const uint8> Nonce, const TArrayView<const uint8> AuthTag, EPlatformCryptoResult& OutResult);

	PLATFORMCRYPTOCONTEXT_API TUniquePtr<IPlatformCryptoEncryptor> CreateEncryptor_AES_256_ECB(const TArrayView<const uint8> Key);
	PLATFORMCRYPTOCONTEXT_API TUniquePtr<IPlatformCryptoEncryptor> CreateEncryptor_AES_256_CBC(const TArrayView<const uint8> Key, const TArrayView<const uint8> InitializationVector);
	PLATFORMCRYPTOCONTEXT_API TUniquePtr<IPlatformCryptoEncryptor> CreateEncryptor_AES_256_GCM(const TArrayView<const uint8> Key, const TArrayView<const uint8> Nonce);

	PLATFORMCRYPTOCONTEXT_API TUniquePtr<IPlatformCryptoDecryptor> CreateDecryptor_AES_256_ECB(const TArrayView<const uint8> Key);
	PLATFORMCRYPTOCONTEXT_API TUniquePtr<IPlatformCryptoDecryptor> CreateDecryptor_AES_256_CBC(const TArrayView<const uint8> Key, const TArrayView<const uint8> InitializationVector);
	PLATFORMCRYPTOCONTEXT_API TUniquePtr<IPlatformCryptoDecryptor> CreateDecryptor_AES_256_GCM(const TArrayView<const uint8> Key, const TArrayView<const uint8> Nonce, const TArrayView<const uint8> AuthTag);

public:

	/**
	 * Sign a message with RS256.
	 *
	 * @param Message The message
	 * @param Signature The resulting signature
	 * @param Key Handle to the RSA key
	 *
	 * @return Whether the sign operation was successful
	 */
	PLATFORMCRYPTOCONTEXT_API bool DigestSign_RS256(const TArrayView<const uint8> Message, TArray<uint8>& Signature, FRSAKeyHandle Key);

	/**
	 * Verify a hashed PS256 message with a signature.
	 *
	 * @param Message The hashed message
	 * @param Signature Signature to verify with
	 * @param PKCS1Key Handle to the RSA key in PKCS1 format
	 *
	 * @return Whether the verify operation was successful
	 */
	PLATFORMCRYPTOCONTEXT_API bool DigestVerify_PS256(const TArrayView<const char> Message, const TArrayView<const uint8> Signature, const TArrayView<const uint8> PKCS1Key);

	/**
	 * Verify a hashed RS256 message with a signature.
	 *
	 * @param Message The hashed message
	 * @param Signature Signature to verify with
	 * @param Key Handle to the RSA key
	 *
	 * @return Whether the verify operation was successful
	 */
	PLATFORMCRYPTOCONTEXT_API bool DigestVerify_RS256(const TArrayView<const uint8> Message, const TArrayView<const uint8> Signature, FRSAKeyHandle Key);

	/**
	 * Generate an RSA key with the number of key bits.
	 *
	 * @param InNumKeyBits Number of key bits
	 * @param OutPublicExponent Array for the public exponent
	 * @param OutPrivateExponent Array for the private exponent
	 * @param OutModulus Array for the modulus
	 *
	 * @return Whether the generate operation was successful
	 */
	PLATFORMCRYPTOCONTEXT_API bool GenerateKey_RSA(const int32 InNumKeyBits, TArray<uint8>& OutPublicExponent, TArray<uint8>& OutPrivateExponent, TArray<uint8>& OutModulus);

	/**
	 * Create an RSA key from a binary public/private exponent and modulus.
	 *
	 * @param PublicExponent Binary key public exponent
	 * @param PrivateExponent Binary key private exponent
	 * @param Modulus Binary key modulus
	 *
	 * @return The created RSA key
	 */
	PLATFORMCRYPTOCONTEXT_API FRSAKeyHandle CreateKey_RSA(const TArrayView<const uint8> PublicExponent, const TArrayView<const uint8> PrivateExponent, const TArrayView<const uint8> Modulus);

	/**
	 * Get the RSA public key from a PEM format string.
	 *
	 * @param PemSource Key in PEM format (base64)
	 *
	 * @return The RSA public key
	 */
	PLATFORMCRYPTOCONTEXT_API FRSAKeyHandle GetPublicKey_RSA(const FStringView PemSource);

	/**
	 * Destroy and free the RSA key.
	 *
	 * @param Key Handle to the key
	 */
	PLATFORMCRYPTOCONTEXT_API void DestroyKey_RSA(FRSAKeyHandle Key);

	/**
	 * Get the RSA key modulus size in bytes.
	 *
	 * @param Key Handle to the key
	 *
	 * @return The modulus size in bytes
	 */
	PLATFORMCRYPTOCONTEXT_API int32 GetKeySize_RSA(FRSAKeyHandle Key);

	/**
	 * Get the maximum data size.
	 *
	 * @param Key Handle to the key
	 *
	 * @return The maximum data size in bytes
	 */
	PLATFORMCRYPTOCONTEXT_API int32 GetMaxDataSize_RSA(FRSAKeyHandle Key);

	/**
	 * Encrypt a source with a public RSA key.
	 *
	 * @param Source Binary source message
	 * @param Dest Array for the encrypted data
	 * @param Key Handle to the key
	 *
	 * @return Number of encrypted bytes
	 */
	PLATFORMCRYPTOCONTEXT_API int32 EncryptPublic_RSA(TArrayView<const uint8> Source, TArray<uint8>& Dest, FRSAKeyHandle Key);

	/**
	 * Encrypt a source with a private RSA key.
	 *
	 * @param Source Binary source message
	 * @param Dest Array for the encrypted data
	 * @param Key Handle to the key
	 *
	 * @return Number of encrypted bytes
	 */
	PLATFORMCRYPTOCONTEXT_API int32 EncryptPrivate_RSA(TArrayView<const uint8> Source, TArray<uint8>& Dest, FRSAKeyHandle Key);

	/**
	 * Decrypt a source with a public RSA key.
	 *
	 * @param Source Binary source message
	 * @param Dest Array for the encrypted data
	 * @param Key Handle to the key
	 *
	 * @return Number of decrypted bytes
	 */
	PLATFORMCRYPTOCONTEXT_API int32 DecryptPublic_RSA(TArrayView<const uint8> Source, TArray<uint8>& Dest, FRSAKeyHandle Key);

	/**
	 * Decrypt a source with a private RSA key.
	 *
	 * @param Source Binary source message
	 * @param Dest Array for the encrypted data
	 * @param Key Handle to the key
	 *
	 * @return Number of decrypted bytes
	 */
	PLATFORMCRYPTOCONTEXT_API int32 DecryptPrivate_RSA(TArrayView<const uint8> Source, TArray<uint8>& Dest, FRSAKeyHandle Key);

public:

	/**
	 * Create random bytes.
	 *
	 * @param OutData Array for the random bytes
	 *
	 * @return The platform crypto result enum
	 */
	PLATFORMCRYPTOCONTEXT_API EPlatformCryptoResult CreateRandomBytes(const TArrayView<uint8> OutData);

	/**
	 * Create pseudo random bytes.
	 *
	 * @param OutData Array for the pseudo random bytes
	 *
	 * @return The platform crypto result enum
	 */
	PLATFORMCRYPTOCONTEXT_API EPlatformCryptoResult CreatePseudoRandomBytes(const TArrayView<uint8> OutData);

public:

	/**
	 * Create a new SHA256 hasher.
	 *
	 * @return Reference to the hasher
	 */
	PLATFORMCRYPTOCONTEXT_API FSHA256Hasher CreateSHA256Hasher();

	/**
	 * Calculate the SHA256 hash of a message.
	 *
	 * @param Source Binary source data  
	 * @param OutHash Array to store the hash in
	 *
	 * @return Whether the hash operation was successful
	 */
	PLATFORMCRYPTOCONTEXT_API bool CalcSHA256(const TArrayView<const uint8> Source, TArray<uint8>& OutHash);
};

typedef FEncryptionContextOpenSSL FEncryptionContext;
