// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"
#include "Misc/ScopeRWLock.h"

/**
 * Module interface for cryptographic functionality. Users should generally go through the FEncryptionContext API.
 */
class IPlatformCryptoContext : public IModuleInterface
{

public:

	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline IPlatformCryptoContext& Get()
	{
		return FModuleManager::LoadModuleChecked< IPlatformCryptoContext >( "PlatformCryptoContext" );
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded( "PlatformCryptoContext" );
	}

	/**
	 * User data access.
	 */
	template<typename T>
	void SetUserData(TSharedPtr<T>& InUserData)
	{
		FRWScopeLock Lock(UserDataLock, SLT_Write);
		UserData = StaticCastSharedPtr<void>(InUserData);
	}

	template<typename T>
	TSharedPtr<T> GetUserData() const
	{
		FRWScopeLock Lock(UserDataLock, SLT_ReadOnly);
		return StaticCastSharedPtr<T>(UserData);
	}

protected:

	mutable FRWLock UserDataLock;
	TSharedPtr<void> UserData;
};
