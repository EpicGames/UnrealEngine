// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using UnrealBuildTool;

[Obsolete("Deprecated in UE5.6 - Use PlatformCryptoContext instead.")]
public class PlatformCryptoOpenSSL : ModuleRules
{
	public PlatformCryptoOpenSSL(ReadOnlyTargetRules Target) : base(Target)
	{
		Type = ModuleType.External;
	}
}
