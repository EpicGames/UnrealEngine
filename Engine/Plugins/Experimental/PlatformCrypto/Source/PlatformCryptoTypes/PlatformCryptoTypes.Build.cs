// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class PlatformCryptoTypes : ModuleRules
	{
		public PlatformCryptoTypes(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
				});
		}
	}
}
