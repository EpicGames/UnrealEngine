// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlatformCryptoTypes.h"
#include "Modules/ModuleManager.h"

class FPlatformCryptoTypesModule : public IModuleInterface
{
public:
	virtual void StartupModule() override
	{
	}

	virtual void ShutdownModule() override
	{
	}
};

IMPLEMENT_MODULE(FPlatformCryptoTypesModule, PlatformCryptoTypes)

FAESBlockEncryptionHelper::FAESBlockEncryptionHelper(const int32 InBlockSize)
	: BlockSize(InBlockSize)
{
	check(InBlockSize > 0);
}

EPlatformCryptoResult FAESBlockEncryptionHelper::Update(const TArrayView<const uint8> InDataBuffer, const TFunctionRef<FBlockHandlerFunctionSignature>& UpdateHandlingFunction, const TArrayView<uint8> OutDataBuffer, int32& OutBytesWritten)
{
	OutBytesWritten = 0;

	// We need more than a block to be able to write out anything
	if (LeftoverDataBuffer.Num() + InDataBuffer.Num() <= BlockSize)
	{
		// Don't have enough to write a block, just add it to our buffer for our next Update/Finalize
		LeftoverDataBuffer.Append(InDataBuffer.GetData(), InDataBuffer.Num());
		return EPlatformCryptoResult::Success;
	}

	// Make a mutable buffer objects that we will reassign as as finish processing data
	TArrayView<const uint8> PlaintextLeftToProcess = InDataBuffer;
	TArrayView<uint8> OutDataBufferAvailable = OutDataBuffer;

	// Encrypt / Decrypt any previously-held chunk, potentially filling it in with new data
	if (LeftoverDataBuffer.Num() > 0)
	{
		const int32 AmountToTake = BlockSize - LeftoverDataBuffer.Num();
		check(AmountToTake >= 0);

		if (AmountToTake > 0)
		{
			// Copy data into our buffer
			LeftoverDataBuffer.Append(PlaintextLeftToProcess.GetData(), AmountToTake);

			// Advance our data by how much we copied out
			PlaintextLeftToProcess = TArrayView<const uint8>(PlaintextLeftToProcess.GetData() + AmountToTake, PlaintextLeftToProcess.Num() - AmountToTake);
		}

		int32 BytesWritten = 0;
		const EPlatformCryptoResult UpdateResult = UpdateHandlingFunction(LeftoverDataBuffer, OutDataBufferAvailable, BytesWritten);
		if (UpdateResult != EPlatformCryptoResult::Success)
		{
			return UpdateResult;
		}

		// Advance our output buffer and shrink size by how much we wrote
		OutDataBufferAvailable = TArrayView<uint8>(OutDataBufferAvailable.GetData() + BytesWritten, OutDataBufferAvailable.Num() - BytesWritten);

		// Write out how much data we've written
		OutBytesWritten += BytesWritten;

		// Reset our buffer (but do not free underlying memory)
		LeftoverDataBuffer.Reset();
	}

	// Encrypt / Decrypt as many new chunks as we can, leaving one
	while (PlaintextLeftToProcess.Num() > BlockSize)
	{
		check(OutDataBufferAvailable.Num() >= BlockSize);

		int32 BytesWritten = 0;
		const EPlatformCryptoResult UpdateResult = UpdateHandlingFunction(TArrayView<const uint8>(PlaintextLeftToProcess.GetData(), BlockSize), OutDataBufferAvailable, BytesWritten);
		if (UpdateResult != EPlatformCryptoResult::Success)
		{
			return UpdateResult;
		}

		OutBytesWritten += BytesWritten;

		// Advance our data by a block
		PlaintextLeftToProcess = TArrayView<const uint8>(PlaintextLeftToProcess.GetData() + BytesWritten, PlaintextLeftToProcess.Num() - BytesWritten);

		// Advance our output buffer by how much we wrote
		OutDataBufferAvailable = TArrayView<uint8>(OutDataBufferAvailable.GetData() + BytesWritten, OutDataBufferAvailable.Num() - BytesWritten);
	}

	// Fill in any leftovers into our mismatched-size buffer
	if (PlaintextLeftToProcess.Num() > 0)
	{
		LeftoverDataBuffer.Append(PlaintextLeftToProcess.GetData(), PlaintextLeftToProcess.Num());
	}

	return EPlatformCryptoResult::Success;
}

EPlatformCryptoResult FAESBlockEncryptionHelper::Finalize(const TFunctionRef<FBlockHandlerFunctionSignature>& FinalizeHandlingFunction, const TArrayView<uint8> OutDataBuffer, int32& OutBytesWritten)
{
	OutBytesWritten = 0;

	// If we have no data, it means that Update was never called (which is fine)
	// If update has never been called, we don't have to do anything
	if (LeftoverDataBuffer.Num() > 0)
	{
		// Process our final chunk of data
		int32 BytesWritten = 0;
		EPlatformCryptoResult FinalResult = FinalizeHandlingFunction(LeftoverDataBuffer, OutDataBuffer, BytesWritten);
		if (FinalResult != EPlatformCryptoResult::Success)
		{
			return FinalResult;
		}

		// Write out how much data we wrote
		OutBytesWritten += BytesWritten;

		// Reset our buffer (but do not free underlying memory)
		LeftoverDataBuffer.Reset();
	}

	return EPlatformCryptoResult::Success;
}
