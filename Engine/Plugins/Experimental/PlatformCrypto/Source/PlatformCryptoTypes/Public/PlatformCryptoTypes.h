// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Stats/Stats.h" // IWYU pragma: keep

#define UE_API PLATFORMCRYPTOTYPES_API

/**
 * Generic result type for cryptographic functions.
 */
enum class EPlatformCryptoResult
{
	Success,
	Failure
};

/**
 * Stat group for implementations to use.
 */
DECLARE_STATS_GROUP(TEXT("PlatformCrypto"), STATGROUP_PlatformCrypto, STATCAT_Advanced);

/**
 * Instance of a encryptor used to progressively encrypt a message in chunks. This is useful when you may not have all
 * data loaded available at one time.
 *
 * Plaintext should be fed into one or more calls to Update, and then Finalize should be called. It is invalid to call
 * Update after Finalize has been called.
 */
class IPlatformCryptoEncryptor
{
public:
	virtual ~IPlatformCryptoEncryptor() = default;

	/**
	 * Get the name of the encryption cipher for this object
	 *
	 * @return Name of our cipher
	 */
	virtual FName GetCipherName() const = 0;

	/**
	 * Get the block size of the encryption cipher for this object. It is best when input data is a multiple of this.
	 *
	 * @return Block size for our cipher
	 */
	virtual int32 GetCipherBlockSizeBytes() const = 0;

	/**
	 * Get the initialization vector size of the encryption cipher for this object.
	 *
	 * @return Block size for our cipher
	 */
	virtual int32 GetCipherInitializationVectorSizeBytes() const = 0;

	/**
	 * Get the required size in bytes for the GenerateAuthTag call's OutAuthTag buffer
	 *
	 * @return The minimum amount of space required to successfully call GetAuthTag with
	 */
	virtual int32 GetCipherAuthTagSizeBytes() const = 0;

	/**
	 * For ciphers that support authentication tags, you may call this function to retreive the AuthTag after Finalize is called successfully. This will
	 * fail for ciphers that do not support AuthTags.
	 *
	 * @param OutAuthTag The Auth Tag used to validate integrity of the the encrypted data during decryption. Must be at least GetAuthTagSizeBytes in size or GetAuthTag will fail.
	 * @param OutAuthTagBytesWritten The amount of bytes written to OutAuthTag. This can be from 0 to GetAuthTagSizebytes
	 */
	virtual EPlatformCryptoResult GenerateAuthTag(const TArrayView<uint8> OutAuthTag, int32& OutAuthTagBytesWritten) const = 0;

	/**
	 * Get the amount of bytes required to safely store the output of a call to Update for a particular Plaintext value
	 *
	 * @param Plaintext The next plaintext data to be encrypted by Update
	 * @return The minimum amount of space required to successfully call Update with
	 */
	virtual int32 GetUpdateBufferSizeBytes(const TArrayView<const uint8> Plaintext) const = 0;

	/**
	 * Encrypt one or more blocks of data. OutCiphertext must be at least size of the result of GetUpdateBufferSizeBytes, or it will fail.  If Plaintext
	 * is less than the size of a block, OutCiphertext may or may not be modified until more data is provided in a future call to Update.
	 *
	 * @param Plaintext The Data to be encrypted and placed into OutCiphertext
	 * @param OutCiphertext The encrypted output of Plaintext. May not be written to if Plaintext is less than block size. Must be at least GetUpdateBufferSizeBytes in size or the call will fail
	 * @param OutCiphertextBytesWritten The amount of data that was written. This can be from 0 to (size of plaintext + blocksize - 1).
	 * @return Success if all input was valid and was encrypted successfully, or Failure otherwise
	 */
	virtual EPlatformCryptoResult Update(const TArrayView<const uint8> Plaintext, const TArrayView<uint8> OutCiphertext, int32& OutCiphertextBytesWritten) = 0;

	/**
	 * Get the amount of bytes required to safely store the OutCipherText buffer of a call to Finalize
	 *
	 * @return The minimum amount of space required to successfully call Finalize with
	 */
	virtual int32 GetFinalizeBufferSizeBytes() const = 0;

	/**
	 * Finalizes encryption of data previously passed in Update. If data was not aligned to blocksize, this will write the final chunk of data
	 * including any padding.
	 *
	 * @param OutCiphertext The final encrypted block, if there is more data to be written. Must be at least GetFinalizeBufferSizeBytes in size or Finalize will fail
	 * @param OutCiphertextBytesWritten The amount of data that was written. This can be from 0 to GetFinalizeBufferSizeBytes
	 * @return Success if input was valid and the final block was encrypted successfully, or Failure otherwise
	 */
	virtual EPlatformCryptoResult Finalize(const TArrayView<uint8> OutCiphertext, int32& OutCiphertextBytesWritten) = 0;

	/**
	 * Resets block cipher. This allows to reuse encryption cipher with the same key for different stream with different InitializationVector.
	 *
	 * @param Optional IV bytes. Will be used only for block modes that requires InitializationVector
	 * @return Success if cipher was reset successfully, or Failure otherwise
	 */
	virtual EPlatformCryptoResult Reset(const TArrayView<const uint8> InitializationVector) = 0;
};

/**
 * Instance of a decryptor used to progressively decrypt a message in chunks. This is useful when you may not have all
 * data loaded available at one time.
 *
 * Encrypted ciphertext should be fed into one or more calls to Update, and then Finalize should be called. It is invalid
 * to call Update after Finalize has been called.
 */
class IPlatformCryptoDecryptor
{
public:
	virtual ~IPlatformCryptoDecryptor() = default;

	/**
	 * Get the name of the decryption cipher for this object
	 *
	 * @return Name of our cipher
	 */
	virtual FName GetCipherName() const = 0;

	/**
	 * Get the block size of the decryption cipher for this object. It is best when input data is a multiple of this.
	 *
	 * @return Block size for our cipher
	 */
	virtual int32 GetCipherBlockSizeBytes() const = 0;

	/**
	 * Get the initialization vector size of the decryption cipher for this object.
	 *
	 * @return Block size for our cipher
	 */
	virtual int32 GetCipherInitializationVectorSizeBytes() const = 0;

	/**
	 * Get the required size in bytes for the GenerateAuthTag call's OutAuthTag buffer
	 *
	 * @return The minimum amount of space required to successfully call GetAuthTag with
	 */
	virtual int32 GetCipherAuthTagSizeBytes() const = 0;

	/**
	 * For ciphers that require authentication tags, you must call this function to set the AuthTag for the encrypted data before you may call Finalize.
	 *
	 * @param AuthTag The Auth Tag used to validate integrity of the the encrypted data during decryption.
	 */
	virtual EPlatformCryptoResult SetAuthTag(const TArrayView<const uint8> AuthTag) = 0;

	/**
	 * Get the minimum required out-data size for a particular Ciphertext value
	 *
	 * @param Ciphertext The next Ciphertext data to be decrypted by Update
	 * @return The minimum amount of space required to successfully call Update with
	 */
	virtual int32 GetUpdateBufferSizeBytes(const TArrayView<const uint8> Ciphertext) const = 0;

	/**
	 * Decrypt one or more blocks of data. OutPlaintext must be at least size of the result of GetUpdateBufferSizeBytes, or it will fail.  If Ciphertext
	 * is less than the size of a block, OutPlaintext may or may not be modified until more data is provided in a future call to Update.
	 *
	 * @param Ciphertext The Data to be decrypted and placed into OutPlaintext
	 * @param OutPlaintext The decrypted output of Ciphertext. May not be written to if Ciphertext is less than block size. Must be at least GetUpdateBufferSizeBytes in size or the call will fail
	 * @param OutPlaintextBytesWritten The amount of data that was written. This can be from 0 to (size of Ciphertext + blocksize - 1).
	 * @return Success if all input was valid and was decrypted successfully, or Failure otherwise
	 */
	virtual EPlatformCryptoResult Update(const TArrayView<const uint8> Ciphertext, const TArrayView<uint8> OutPlaintext, int32& OutPlaintextBytesWritten) = 0;

	/**
	 * Get the minimum required out-data size for the finalize call
	 *
	 * @return The minimum amount of space required to successfully call Finalize with
	 */
	virtual int32 GetFinalizeBufferSizeBytes() const = 0;

	/**
	 * Finalizes decryption of data previously passed in Update. If data was not aligned to blocksize, this will write the final chunk of data
	 * including any padding.
	 *
	 * @param OutPlaintext The final decrypted block. May not be written to if there was nothing left to write. Must be at least GetCipherBlockSize in size or the call will fail
	 * @param OutPlaintextBytesWritten The amount of data that was written. This can be from 0 to blocksize - 1
	 * @return Success if input was valid and the final block was decrypted successfully, or Failure otherwise
	 */
	virtual EPlatformCryptoResult Finalize(const TArrayView<uint8> OutPlaintext, int32& OutPlaintextBytesWritten) = 0;

	/**
	 * Resets block cipher. This allows to reuse decryption cipher with the same key for different stream with different InitializationVector.
	 *
	 * @param Optional IV bytes. Will be used only for block modes that requires InitializationVector
	 * @return Success if cipher was reset successfully, or Failure otherwise
	 */
	virtual EPlatformCryptoResult Reset(const TArrayView<const uint8> InitializationVector) = 0;
};

/**
 * Helper class to ensure that only multiples of block-size get passed into our Encrypt/Decrypt functions.
 * This class will hold onto the last-full or partial block until another call to Update or Finalize happens.
 */
class FAESBlockEncryptionHelper
{
public:
	UE_API FAESBlockEncryptionHelper(const int32 BlockSize);

	using FBlockHandlerFunctionSignature = EPlatformCryptoResult(const TArrayView<const uint8> /*InDataBuffer*/, const TArrayView<uint8> /*OutDataBuffer*/, int32& /*OutBytesWritten*/);

	/**
	 * Add text to potentialy be processed by the UpdateHandlingFunction. If there is not enough text, the UpdateHandlingFunction may not be called.
	 *
	 * @param InDataBuffer Buffer pointing to the data that needs to be encrypted/decrypted
	 * @param UpdateHandlingFunction A function that can encrypt/decrypt a complete block of text
	 * @param OutDataBuffer A buffer that can hold the complete encrypted/decrypted data from UpdateHandlingFunction
	 * @param OutBytesWritten The amount of bytes written to OutDataBuffer
	 * @return Success if the data could be successfully processed, or Failure otherwise.
	 */
	UE_API EPlatformCryptoResult Update(const TArrayView<const uint8> InDataBuffer, const TFunctionRef<FBlockHandlerFunctionSignature>& UpdateHandlingFunction, const TArrayView<uint8> OutDataBuffer, int32& OutBytesWritten);

	/**
	 * Process any remaining block of text that was previously sent to Update. If there are no blocks remaining, the FinalizeHandlingFunction may not be called.
	 *
	 * @param FinalizeHandlingFunction A function that can encrypt/decrypt a complete block of text
	 * @param OutDataBuffer A buffer that can hold the complete encrypted/decrypted data from FinalizeHandlingFunction
	 * @param OutBytesWritten The amount of bytes written to OutDataBuffer
	 * @return Success if the data could be successfully processed, or Failure otherwise.
	 */
	UE_API EPlatformCryptoResult Finalize(const TFunctionRef<FBlockHandlerFunctionSignature>& FinalizeHandlingFunction, const TArrayView<uint8> OutDataBuffer, int32& OutBytesWritten);

protected:
	/** A buffer to store data waiting to be encrypted/decrypted */
	TArray<uint8> LeftoverDataBuffer;
	/** The size of a complete block of data */
	int32 BlockSize = 0;
};

/** Structure with public/private key pair. */
struct FCryptoKeyPair
{
	TArray<uint8> PrivateKey;
	TArray<uint8> PublicKey;
};
#undef UE_API
